[CoreRedirects]

+EnumRedirects=(OldName="/Script/MaterialValidation.EMaterialInstanceColumnType",NewName="/Script/MaterialValidation.EMaterialPropertyCategory")
