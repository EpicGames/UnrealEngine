// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Materials/MaterialInstanceBasePropertyOverrides.h"
#include "Templates/TypeHash.h"

#include "MaterialValidationAssetData.generated.h"

/** 
 * Material instance properties that trigger a static permutation. 
 * This is a selected subset of FMaterialInstanceBasePropertyOverrides and may need updating if anything is added to that struct.
 */
USTRUCT()
struct FStaticPermutationProperties
{
	GENERATED_BODY()

	UPROPERTY()
	uint8 TwoSided : 1;
	UPROPERTY()
	uint8 bIsThinSurface : 1;
	UPROPERTY()
	uint8 DitheredLODTransition : 1;
	UPROPERTY()
	uint8 bCastDynamicShadowAsMasked:1;
	UPROPERTY()
	uint8 bOutputTranslucentVelocity : 1;
	UPROPERTY()
	uint8 bHasPixelAnimation : 1;
	UPROPERTY()
	uint8 bEnableTessellation : 1;
	UPROPERTY()
	TEnumAsByte<EBlendMode> BlendMode;
	UPROPERTY()
	TEnumAsByte<EMaterialShadingModel> ShadingModel;
	UPROPERTY()
	float OpacityMaskClipValue;
	UPROPERTY()
	uint32 UsageFlags;

	FStaticPermutationProperties();

	bool operator==(const FStaticPermutationProperties& Other) const = default;

	friend uint32 GetTypeHash(const FStaticPermutationProperties& Arg);
};

/**
 * Flags that enable override of the properties in FStaticPermutationProperties.
 * This is a selected subset of FMaterialInstanceBasePropertyOverrides and may need updating if anything is added to that struct.
 */
USTRUCT()
struct FStaticPermutationPropertyOverrideFlags
{
	GENERATED_BODY()

	UPROPERTY()
	uint8 bOverride_OpacityMaskClipValue : 1;
	UPROPERTY()
	uint8 bOverride_BlendMode : 1;
	UPROPERTY()
	uint8 bOverride_ShadingModel : 1;
	UPROPERTY()
	uint8 bOverride_DitheredLODTransition : 1;
	UPROPERTY()
	uint8 bOverride_CastDynamicShadowAsMasked : 1;
	UPROPERTY()
	uint8 bOverride_TwoSided : 1;
	UPROPERTY()
	uint8 bOverride_bIsThinSurface : 1;
	UPROPERTY()
	uint8 bOverride_OutputTranslucentVelocity : 1;
	UPROPERTY()
	uint8 bOverride_bHasPixelAnimation : 1;
	UPROPERTY()
	uint8 bOverride_bEnableTessellation : 1;
	UPROPERTY()
	uint32 Override_UsageFlags;

	FStaticPermutationPropertyOverrideFlags();
};

/** 
 * Extracted data from UMaterial to be used for predicting the shader permutation ID of UMaterial and child UMaterialInstances. 
 * This is intended for storage external to the UMaterial so that we don't need to load it.
 * In future this may be better in the AssetData, but we will need to resave all assets before we can rely on that.
 */
USTRUCT()
struct FMaterialValidationAssetData_Material
{
	GENERATED_BODY()

	UPROPERTY()
 	TArray<uint32> StaticSwitchValues;
 	UPROPERTY()
 	TArray<uint32> ComponentMaskValues;
	UPROPERTY()
	FStaticPermutationProperties StaticProperties;
	UPROPERTY()
	uint32 StaticPropertyLayoutHash = 0;
	UPROPERTY()
	uint32 MaterialLayerHash = 0;
	UPROPERTY()
	uint32 PermutationHash = 0;
	UPROPERTY()
	uint16 StaticSwitchNum = 0;
	UPROPERTY()
	uint16 ComponentMaskNum = 0;
};

/**
 * Extracted data from UMaterialInstance to be used for predicting the shader permutation.
 * This is intended for storage external to the UMaterialInstance so that we don't need to load it.
 * In future this may be better in the AssetData, but we will need to resave all assets before we can rely on that.
 */
USTRUCT()
struct FMaterialValidationAssetData_MaterialInstance
{
	GENERATED_BODY()

 	UPROPERTY()
 	TArray<uint32> StaticSwitchOverrideMask;
 	UPROPERTY()
 	TArray<uint32> StaticSwitchOverrideValues;
 	UPROPERTY()
 	TArray<uint32> ComponentMaskOverrideMask;
 	UPROPERTY()
 	TArray<uint32> ComponentMaskOverrideValues;
	UPROPERTY()
	FStaticPermutationProperties StaticProperties;
	UPROPERTY()
	FStaticPermutationPropertyOverrideFlags StaticPropertyOverrideFlags;
	UPROPERTY()
	uint32 MaterialLayerHash = 0;
	UPROPERTY()
	uint32 PermutationHash = 0;
};

namespace MaterialValidation
{
	uint32 BuildPermutationHash(
		FStaticPermutationProperties const& InStaticProperties,
		TConstArrayView<uint32> InStaticSwitchValues,
		TConstArrayView<uint32> InComponentMaskValues,
		uint32 InMaterialLayerHash
	);
}
