// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialValidationChangelistTab.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "ISourceControlChangelistState.h"
#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "MaterialEditingLibrary.h"
#include "MaterialValidationGroup.h"
#include "MaterialValidationLibrary.h"
#include "MaterialValidators.h"
#include "MaterialValidationLibraryTypes.h"
#include "MaterialValidationToolkitShared.h"
#include "Materials/Material.h"
#include "Materials/MaterialInterface.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialFunctionInstance.h"
#include "Materials/MaterialFunctionMaterialLayer.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Misc/PackageName.h"
#include "Misc/ScopedSlowTask.h"
#include "SourceControlOperations.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "MaterialValidationChangelists"

namespace MaterialValidation {

/** One row in the changelists tab base-material list view. The final entry is a synthetic summary row. */
struct FBaseMaterialRow
{
	/** The row holds the summary of total deltas. */
	bool bIsSummaryRow = false;
	/** Path to the UMaterial asset. Not set if this is the summary row. */
	FSoftObjectPath AssetPath;
	/** Number of material instances in the validation hierarchy for this base material. */
	int32 InstanceCount = 0;
	/** Number of changelist files belonging to this base material hierarchy (including the base material itself). */
	int32 ChangelistInstanceCount = 0;
	/** Shader delta computed by the fast (workspace-only) validation. Unset until that mode has been run. */
	TOptional<int32> FastShaderDelta;
	/** Shader delta computed by the slow (depot vs workspace) validation. Unset until that mode has been run. */
	TOptional<int32> SlowShaderDelta;

	static const FName ColumnId_AssetPath;
	static const FName ColumnId_Instances;
	static const FName ColumnId_ChangelistInstances;
	static const FName ColumnId_FastDelta;
	static const FName ColumnId_SlowDelta;
};

const FName FBaseMaterialRow::ColumnId_AssetPath("CID_AssetPath");
const FName FBaseMaterialRow::ColumnId_Instances("CID_Instances");
const FName FBaseMaterialRow::ColumnId_ChangelistInstances("CID_ChangelistInstances");
const FName FBaseMaterialRow::ColumnId_FastDelta("CID_FastDelta");
const FName FBaseMaterialRow::ColumnId_SlowDelta("CID_SlowDelta");

/** A row in the changelists tab base-material SListView. */
class SBaseMaterialTableRow : public SMultiColumnTableRow<TSharedPtr<FBaseMaterialRow>>
{
	TSharedPtr<FBaseMaterialRow> Item;

public:
	SLATE_BEGIN_ARGS(SBaseMaterialTableRow) {}
		SLATE_ARGUMENT(TSharedPtr<FBaseMaterialRow>, Item)
	SLATE_END_ARGS()

	/** Builds the row. */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable)
	{
		Item = InArgs._Item;
		SMultiColumnTableRow<TSharedPtr<FBaseMaterialRow>>::Construct(FSuperRowType::FArguments(), InOwnerTable);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (!Item.IsValid())
		{
			return SNew(STextBlock).Text(FText::GetEmpty());
		}
		if (ColumnName == FBaseMaterialRow::ColumnId_AssetPath)
		{
			if (Item->bIsSummaryRow)
			{
				return SNew(STextBlock)
					.Text(LOCTEXT("TotalRow", "Total"))
					.Font(FAppStyle::GetFontStyle("BoldFont"));
			}
			return GenerateAssetPathWidget(Item->AssetPath, FSlateColor::UseForeground());
		}
		if (ColumnName == FBaseMaterialRow::ColumnId_Instances)
		{
			if (Item->bIsSummaryRow)
			{
				return SNew(STextBlock).Text(FText::GetEmpty());
			}
			return SNew(STextBlock).Text(FText::AsNumber(Item->InstanceCount));
		}
		if (ColumnName == FBaseMaterialRow::ColumnId_ChangelistInstances)
		{
			if (Item->bIsSummaryRow)
			{
				return SNew(STextBlock).Text(FText::GetEmpty());
			}
			return SNew(STextBlock).Text(FText::AsNumber(Item->ChangelistInstanceCount));
		}
		if (ColumnName == FBaseMaterialRow::ColumnId_FastDelta)
		{
			return MakeDeltaCell(Item->FastShaderDelta);
		}
		if (ColumnName == FBaseMaterialRow::ColumnId_SlowDelta)
		{
			return MakeDeltaCell(Item->SlowShaderDelta);
		}

		return SNew(STextBlock).Text(GUnknownText);
	}

private:
	static TSharedRef<SWidget> MakeDeltaCell(TOptional<int32> Delta)
	{
		if (Delta.IsSet())
		{
			return SNew(STextBlock)
				.Text(GetTextSignedInt(*Delta))
				.ColorAndOpacity(GetDeltaColor(*Delta));
		}
		return SNew(STextBlock).Text(GEmptyText);
	}
};

/** Represents a single pending source control changelist for display in the combo box. */
struct FChangelistEntry
{
	FSourceControlChangelistStatePtr ChangelistState;
	FText DisplayText;
};

/**
 * Widget for the Changelists tab. The user can select a pending changelist and run the permutation analysis
 * to get direct results. This is intended to be more actionable than the generic pre-submit validation.
 */
class SChangelistsTab : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SChangelistsTab) {}
	SLATE_END_ARGS()

	/** Builds the slate view. */
	void Construct(FArguments const& InArgs)
	{
		ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(4.f, 4.f, 4.f, 0.f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.f, 0.f, 8.f, 0.f)
				[
					SNew(SCheckBox)
					.IsChecked_Lambda([this]() { return bFilterOpenedFilesOnly ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
					.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState) { bFilterOpenedFilesOnly = (NewState == ECheckBoxState::Checked); })
					.ToolTipText(LOCTEXT("FilterOpenedOnly_Tooltip", "Only show changelists that have at least one file opened for edit (excluding shelved-only changelists)."))
					[
						SNew(STextBlock).Text(LOCTEXT("FilterOpenedOnly", "Opened files only"))
					]
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SCheckBox)
					.IsChecked_Lambda([this]() { return bFilterMaterialAssetsOnly ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
					.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState) { bFilterMaterialAssetsOnly = (NewState == ECheckBoxState::Checked); })
					.ToolTipText(LOCTEXT("FilterMaterialsOnly_Tooltip", "Only show changelists that contain at least one material, material instance, or material function asset."))
					[
						SNew(STextBlock).Text(LOCTEXT("FilterMaterialsOnly", "Material assets only"))
					]
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(4.f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					SAssignNew(ChangelistComboBox, SComboBox<TSharedPtr<FChangelistEntry>>)
					.OptionsSource(&ChangelistEntries)
					.OnGenerateWidget_Lambda([](TSharedPtr<FChangelistEntry> Entry) -> TSharedRef<SWidget>
					{
						return SNew(STextBlock)
							.Text(Entry.IsValid() ? Entry->DisplayText : FText::GetEmpty());
					})
					.OnSelectionChanged_Lambda([this](TSharedPtr<FChangelistEntry> Entry, ESelectInfo::Type)
					{
						if (SelectedEntry != Entry)
						{
							SelectedEntry = Entry;
							ResetMaterialsList();
						}
					})
					[
						SNew(STextBlock)
						.Text_Lambda([this]()
						{
							return SelectedEntry.IsValid()
								? SelectedEntry->DisplayText
								: LOCTEXT("NoChangelist", "(no changelist selected)");
						})
					]
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(4.f, 0.f)
				[
					SNew(SButton)
					.Text(LOCTEXT("Refresh", "Refresh"))
					.OnClicked(this, &SChangelistsTab::OnRefreshClicked)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.Text(LOCTEXT("Validate", "Validate"))
					.OnClicked(this, &SChangelistsTab::OnValidateClicked)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(4.f, 0.f)
				[
					SNew(SButton)
					.Text(LOCTEXT("ValidateSlow", "Validate (Slow)"))
					.OnClicked(this, &SChangelistsTab::OnValidateSlowClicked)
				]
			]
			+ SVerticalBox::Slot()
			.FillHeight(1.f)
			.Padding(4.f)
			[
				SNew(SScrollBox)
				+ SScrollBox::Slot()
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SAssignNew(BaseMaterialsList, SListView<TSharedPtr<FBaseMaterialRow>>)
						.HeaderRow(
							SNew(SHeaderRow)
							+ SHeaderRow::Column(FBaseMaterialRow::ColumnId_AssetPath)
								.HAlignHeader(HAlign_Center).HAlignCell(HAlign_Left)
								.DefaultLabel(LOCTEXT("ColAssetPath", "Base Material"))
								.ManualWidth_Lambda([]() { return GAssetNameColumnWidth; })
								.OnWidthChanged_Lambda([](float NewWidth) { GAssetNameColumnWidth = NewWidth; })
							+ SHeaderRow::Column(FBaseMaterialRow::ColumnId_Instances)
								.HAlignHeader(HAlign_Center).HAlignCell(HAlign_Center)
								.DefaultLabel(LOCTEXT("ColInstances", "Instance Count"))
								.FixedWidth(100.f)
							+ SHeaderRow::Column(FBaseMaterialRow::ColumnId_ChangelistInstances)
								.HAlignHeader(HAlign_Center).HAlignCell(HAlign_Center)
								.DefaultLabel(LOCTEXT("ColCLInstances", "Instances in Changelist"))
								.FixedWidth(150.f)
							+ SHeaderRow::Column(FBaseMaterialRow::ColumnId_FastDelta)
								.HAlignHeader(HAlign_Center).HAlignCell(HAlign_Center)
								.DefaultLabel(LOCTEXT("ColFastDelta", "Shader Delta (Fast)"))
								.FixedWidth(130.f)
							+ SHeaderRow::Column(FBaseMaterialRow::ColumnId_SlowDelta)
								.HAlignHeader(HAlign_Center).HAlignCell(HAlign_Center)
								.DefaultLabel(LOCTEXT("ColSlowDelta", "Shader Delta (Slow)"))
								.FixedWidth(130.f)
						)
						.ListItemsSource(&BaseMaterialRows)
						.OnGenerateRow_Lambda([](TSharedPtr<FBaseMaterialRow> Item, const TSharedRef<STableViewBase>& OwnerTable)
						{
							return SNew(SBaseMaterialTableRow, OwnerTable).Item(Item);
						})
						.SelectionMode(ESelectionMode::None)
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.f, 4.f)
					[
						SAssignNew(NotesText, STextBlock)
						.AutoWrapText(true)
					]
				]
			]
		];
	}

private:
	/** Reset the validation output. */
	void ResetMaterialsList()
	{
		BaseMaterialRows.Reset();
		BaseMaterialsList->RequestListRefresh();
		NotesText->SetText(FText::GetEmpty());
	}

	/** Synchronously queries source control for pending changelists, applies active filters, and repopulates the combo box. */
	FReply OnRefreshClicked()
	{
		if (!ISourceControlModule::Get().IsEnabled())
		{
			NotesText->SetText(LOCTEXT("SCDisabled", "Source control is not enabled."));
			return FReply::Handled();
		}

		ISourceControlProvider& Provider = ISourceControlModule::Get().GetProvider();

		TSharedRef<FUpdatePendingChangelistsStatus, ESPMode::ThreadSafe> UpdateOp =	ISourceControlOperation::Create<FUpdatePendingChangelistsStatus>();
		UpdateOp->SetUpdateAllChangelists(true);
		UpdateOp->SetUpdateFilesStates(true);
		Provider.Execute(UpdateOp, EConcurrency::Synchronous);

		TArray<FSourceControlChangelistRef> Changelists = Provider.GetChangelists(EStateCacheUsage::Use);
		TArray<FSourceControlChangelistStateRef> ChangelistStates;
		Provider.GetState(Changelists, ChangelistStates, EStateCacheUsage::Use);

		ChangelistEntries.Reset();
		for (FSourceControlChangelistStateRef const& State : ChangelistStates)
		{
			// Skip changelists with no opened (non-shelved) files.
			if (bFilterOpenedFilesOnly && State->GetFilesStatesNum() == 0)
			{
				continue;
			}

			// Skip changelists with no material related assets.
			if (bFilterMaterialAssetsOnly && !ChangelistHasMaterialAsset(State))
			{
				continue;
			}

			// Truncate description to the first line for compact display.
			FString Description = State->GetDescriptionText().ToString();
			int32 NewlineIndex;
			if (Description.FindChar(TEXT('\n'), NewlineIndex))
			{
				Description.LeftInline(NewlineIndex);
				Description.TrimEndInline();
			}

			TSharedPtr<FChangelistEntry> Entry = MakeShared<FChangelistEntry>();
			Entry->ChangelistState = State;
			Entry->DisplayText = FText::Format(LOCTEXT("EntryFmt", "{0}: {1}"), State->GetDisplayText(), FText::FromString(Description));
			ChangelistEntries.Add(Entry);
		}

		// Sort descending so the most recent (highest-numbered) changelist appears first.
		ChangelistEntries.Sort([](TSharedPtr<FChangelistEntry> const& A, TSharedPtr<FChangelistEntry> const& B)
		{
			int64 const NumA = FCString::Atoi64(*A->ChangelistState->GetDisplayText().ToString());
			int64 const NumB = FCString::Atoi64(*B->ChangelistState->GetDisplayText().ToString());
			return NumA > NumB;
		});

		ChangelistComboBox->RefreshOptions();

		// Try to reselect the previously selected changelist.
		// If it is still present in the new list, keep the selection and leave the results table intact.
		FText const PreviousDisplayText = SelectedEntry.IsValid() ? SelectedEntry->ChangelistState->GetDisplayText() : FText::GetEmpty();
		TSharedPtr<FChangelistEntry>* ReselectedEntry = ChangelistEntries.FindByPredicate([&PreviousDisplayText](TSharedPtr<FChangelistEntry> const& E)
		{
			return E->ChangelistState->GetDisplayText().EqualTo(PreviousDisplayText);
		});

		if (ReselectedEntry)
		{
			SelectedEntry = *ReselectedEntry;
			ChangelistComboBox->SetSelectedItem(SelectedEntry);
		}
		else
		{
			// Selection changed so clear stale results.
			ResetMaterialsList();

			if (ChangelistEntries.Num() > 0)
			{
				ChangelistComboBox->SetSelectedItem(ChangelistEntries[0]);
				SelectedEntry = ChangelistEntries[0];
			}
			else
			{
				SelectedEntry = nullptr;
			}
		}

		return FReply::Handled();
	}

	/** Accumulated output of the collection phase, consumed by both fast and slow validation summary phases. */
	struct FValidateCollectionResult
	{
		/** Base materials whose full permutation set needs re-evaluating. */
		TArray<UMaterial*> BaseMaterials;
		/** Modified objects in the change. Always the local version. */
		TArray<UMaterialInterface*> ModifiedObjects;
		/** Modified objects for the "before" shader count. In slow mode this holds the depot versions. In fast mode this is unused. */
		TArray<UMaterialInterface*> ReplacementObjects;
		/** Depot packages loaded for slow mode; must be marked as garbage after the shader count computation. */
		TArray<UPackage*> DepotPackages;
		/** Filenames of UE assets that could not be loaded from disk. */
		TArray<FString> SkippedAssets;
		/** Number of UE asset files found in the changelist. */
		int32 TotalFilesChecked = 0;
		/** True if the changelist contains material function files. In slow mode these are not analyzed. */
		bool bHasFunctionFiles = false;
		/** Number of changelist files belonging to each base material hierarchy (including the base material itself). */
		TMap<FSoftObjectPath, int32> ChangelistInstanceCounts;
	};

	/** Returns the first validation group whose search path contains InMaterial, or null if none matches. */
	static UMaterialValidationGroup* FindGroupForMaterial(TArray<UMaterialValidationGroup*> const& InGroups, UMaterial* InMaterial)
	{
		for (UMaterialValidationGroup* Group : InGroups)
		{
			bool bIsInGroupPath = false;
			bool bIsInGroup = false;
			UMaterialValidationLibrary::IsMaterialInGroup(Group, InMaterial, bIsInGroupPath, bIsInGroup);
			if (bIsInGroupPath)
			{
				return Group;
			}
		}
		return nullptr;
	}

	/** Returns the first validation group whose material list contains the base material of InInstance, or null if none matches. */
	static UMaterialValidationGroup* FindGroupForMaterialInstance(TArray<UMaterialValidationGroup*> const& InGroups, UMaterialInstanceConstant* InInstance)
	{
		for (UMaterialValidationGroup* Group : InGroups)
		{
			bool bMaterialInGroup = false;
			bool bMaterialPermutationInGroup = false;
			UMaterialValidationLibrary::IsMaterialInstanceInGroup(Group, InInstance, bMaterialInGroup, bMaterialPermutationInGroup);
			if (bMaterialInGroup)
			{
				return Group;
			}
		}
		return nullptr;
	}

	/**
	 * Iterates the files in InFileStates and loads workspace assets to fill OutResult.
	 * When bExpandFunctions is true material function files are expanded to their referencing base materials.
	 * Depot loading is not performed here; call FetchRevisionHistory + LoadDepotVersions separately for slow mode.
	 */
	static void CollectFromChangelist(TArray<UMaterialValidationGroup*> const& InGroups, TArray<FSourceControlStateRef> const& InFileStates, bool bExpandFunctions, FValidateCollectionResult& OutResult)
	{
		for (FSourceControlStateRef const& FileState : InFileStates)
		{
			FString const& Filename = FileState->GetFilename();

			FString PackageName;
			if (!FPackageName::TryConvertFilenameToLongPackageName(Filename, PackageName))
			{
				continue;
			}

			++OutResult.TotalFilesChecked;

			FString const ShortName = FPackageName::GetShortName(PackageName);
			FString const ObjectPath = PackageName + TEXT(".") + ShortName;
			UObject* WorkspaceAsset = StaticLoadObject(UObject::StaticClass(), nullptr, *ObjectPath, nullptr, LOAD_NoWarn | LOAD_Quiet);
			if (WorkspaceAsset == nullptr)
			{
				OutResult.SkippedAssets.Add(FPaths::GetCleanFilename(Filename));
				continue;
			}

			UMaterial* Material = Cast<UMaterial>(WorkspaceAsset);
			UMaterialInstanceConstant* MaterialInstance = Cast<UMaterialInstanceConstant>(WorkspaceAsset);
			UMaterialFunction* MaterialFunction = Cast<UMaterialFunction>(WorkspaceAsset);

			if (Material != nullptr)
			{
				if (FindGroupForMaterial(InGroups, Material))
				{
					OutResult.BaseMaterials.AddUnique(Material);
					OutResult.ChangelistInstanceCounts.FindOrAdd(FSoftObjectPath(Material))++;
					OutResult.ModifiedObjects.Add(Material);
				}
			}
			if (MaterialInstance != nullptr && MaterialInstance->bHasStaticPermutationResource)
			{
				if (FindGroupForMaterialInstance(InGroups, MaterialInstance))
				{
					OutResult.BaseMaterials.AddUnique(MaterialInstance->GetMaterial());
					OutResult.ChangelistInstanceCounts.FindOrAdd(FSoftObjectPath(MaterialInstance->GetMaterial()))++;
					OutResult.ModifiedObjects.Add(MaterialInstance);
				}
			}

			if (MaterialFunction != nullptr)
			{
				OutResult.bHasFunctionFiles = true;

				if (bExpandFunctions)
				{
					// Note that loading all referencing base materials can be slow for widely-used functions.
					// Function derived base materials cannot be analyzed in slow mode for two reasons:
					// * It would probably expand the working set of loaded assets by too much to be useable.
					// * We would need a way to rebind the temporary depot assets into their referenced materials.
					TArray<FAssetData> MaterialAssetDatas;
					UMaterialEditingLibrary::GetMaterialsReferencingFunction(MaterialFunction, MaterialAssetDatas);
					for (FAssetData const& AssetData : MaterialAssetDatas)
					{
						if (UMaterial* BaseMaterial = Cast<UMaterial>(AssetData.GetAsset()))
						{
							OutResult.BaseMaterials.AddUnique(BaseMaterial);
						}
					}
				}
			}
		}
	}

	/** Returns false and sets NotesText if no valid changelist is selected. */
	bool EnsureValidSelection()
	{
		if (!SelectedEntry.IsValid())
		{
			NotesText->SetText(LOCTEXT("NoSelection", "No changelist selected. Press Refresh first."));
			return false;
		}
		if (!SelectedEntry->ChangelistState.IsValid())
		{
			NotesText->SetText(LOCTEXT("InvalidState", "Changelist state is no longer valid. Press Refresh."));
			return false;
		}
		return true;
	}

	/** Intermediate result produced by the summary phase for each base material. */
	struct FPerMaterialResult
	{
		int32 InstanceCount = 0;
		int32 ChangelistInstanceCount = 0;
		int32 ShaderDelta = 0;
	};

	enum class EValidationMode { Fast, Slow };

	/**
	 * Inserts rows in BaseMaterialRows from PerMaterialResults and refreshes the list view.
	 * Mode selects which delta column (FastShaderDelta or SlowShaderDelta) to populate.
	 */
	void UpdateResults(FValidateCollectionResult const& Collection, TMap<FSoftObjectPath, FPerMaterialResult> const& PerMaterialResults, EValidationMode Mode)
	{
		// Remove the summary row so we can update data rows first.
		if (!BaseMaterialRows.IsEmpty() && BaseMaterialRows.Last()->bIsSummaryRow)
		{
			BaseMaterialRows.Pop();
		}

		// Upsert one row per base material.
		for (UMaterial* Material : Collection.BaseMaterials)
		{
			FSoftObjectPath const Path(Material);

			// Start from the existing row to preserve any existing data.
			TSharedPtr<FBaseMaterialRow>* Existing = BaseMaterialRows.FindByPredicate([&Path](TSharedPtr<FBaseMaterialRow> const& R) { return R->AssetPath == Path; });

			FBaseMaterialRow RowData = Existing ? **Existing : FBaseMaterialRow{};
			RowData.AssetPath = Path;
			if (FPerMaterialResult const* Result = PerMaterialResults.Find(Path))
			{
				RowData.InstanceCount = Result->InstanceCount;
				RowData.ChangelistInstanceCount = Result->ChangelistInstanceCount;
				if (Mode == EValidationMode::Fast)
				{
					RowData.FastShaderDelta = Result->ShaderDelta;
				}
				else
				{
					RowData.SlowShaderDelta = Result->ShaderDelta;
				}
			}

			// Always wrap in a new TSharedPtr so SListView sees each item as changed and regenerates the row widget.
			TSharedPtr<FBaseMaterialRow> Row = MakeShared<FBaseMaterialRow>(RowData);
			if (Existing) 
			{
				*Existing = Row; 
			}
			else
			{
				BaseMaterialRows.Add(Row); 
			}
		}

		// Rebuild the summary row.
		TSharedPtr<FBaseMaterialRow> Summary = MakeShared<FBaseMaterialRow>();
		Summary->bIsSummaryRow = true;
		for (TSharedPtr<FBaseMaterialRow> const& R : BaseMaterialRows)
		{
			if (R->FastShaderDelta.IsSet()) 
			{
				Summary->FastShaderDelta = Summary->FastShaderDelta.Get(0) + *R->FastShaderDelta; 
			}
			if (R->SlowShaderDelta.IsSet()) 
			{ 
				Summary->SlowShaderDelta = Summary->SlowShaderDelta.Get(0) + *R->SlowShaderDelta; 
			}
		}
		BaseMaterialRows.Add(Summary);
		BaseMaterialsList->RequestListRefresh();

		// Update notes text.
		TArray<FText> Notes;
		if (Collection.bHasFunctionFiles)
		{
			Notes.Add(LOCTEXT("FunctionFilesNote", "Note: changelist contains material function(s). Base materials that reference these are not analyzed in slow mode."));
		}
		if (Collection.SkippedAssets.Num() > 0)
		{
			Notes.Add(FText::Format(
				LOCTEXT("SkippedAssetsFmt", "Could not load {0} asset(s): {1}"),
				FText::AsNumber(Collection.SkippedAssets.Num()),
				FText::FromString(FString::Join(Collection.SkippedAssets, TEXT(", ")))));
		}
		NotesText->SetText(FText::Join(FText::FromString(TEXT("\n")), Notes));
	}

	/** Loads assets from the selected changelist and computes the shader permutation delta. */
	FReply OnValidateClicked()
	{
		if (!EnsureValidSelection())
		{
			return FReply::Handled();
		}

		TArray<UMaterialValidationGroup*> Groups;
		UMaterialValidationLibrary::GetAllGroups(Groups, /*bInSyncLoad*/true);

		FValidateCollectionResult Collection;
		CollectFromChangelist(Groups, SelectedEntry->ChangelistState->GetFilesStates(), /*bExpandFunctions=*/true, Collection);

		TMap<FSoftObjectPath, FPerMaterialResult> PerMaterialResults;
		for (UMaterial* BaseMaterial : Collection.BaseMaterials)
		{
			FSoftObjectPath const Path(BaseMaterial);
			if (UMaterialValidationGroup* Group = FindGroupForMaterial(Groups, BaseMaterial))
			{
				FMaterialDatabaseAssetHierarchyInfo HierarchyInfo;
				UMaterialValidationLibrary::GetMaterialHierarchyInfo(Group, Path, HierarchyInfo);
				const int32 InstanceCount = FMath::Max(0, HierarchyInfo.MaterialPaths.Num() - 1);
				const bool bForceLoadObjects = false;
				const int32 Before = UMaterialValidationLibrary::GetShaderCount(Group, BaseMaterial);
				const int32 After  = UMaterialValidationLibrary::GetModifiedShaderCount(Group, BaseMaterial, Collection.ModifiedObjects, {}, bForceLoadObjects);
				const int32 ChangelistCount = Collection.ChangelistInstanceCounts.FindRef(Path);
				PerMaterialResults.Add(Path, {InstanceCount, ChangelistCount, After - Before});
			}
		}

		UpdateResults(Collection, PerMaterialResults, EValidationMode::Fast);
		return FReply::Handled();
	}

	/**
	 * Loads assets from the selected changelist and computes the shader permutation delta using depot versions as the baseline and workspace versions as the modified state.
	 * More accurate than Validate but requires fetching revision history from source control and force-loading all material instances in each affected hierarchy.
	 * Material function files are not analyzed and a note is shown if any are found.
	 */
	FReply OnValidateSlowClicked()
	{
		if (!EnsureValidSelection())
		{
			return FReply::Handled();
		}

		TArray<UMaterialValidationGroup*> Groups;
		UMaterialValidationLibrary::GetAllGroups(Groups, /*bInSyncLoad*/true);

		FScopedSlowTask SlowTask(2.f, LOCTEXT("SlowValidating", "Running slow validation..."));
		SlowTask.MakeDialog(/*bShowCancelButton=*/true);

		SlowTask.EnterProgressFrame(1.f, LOCTEXT("SlowCollecting", "Collecting material assets..."));
		FValidateCollectionResult Collection;
		CollectFromChangelist(Groups, SelectedEntry->ChangelistState->GetFilesStates(), /*bExpandFunctions=*/false, Collection);

		SlowTask.EnterProgressFrame(1.f, LOCTEXT("SlowFetchHistory", "Fetching revision history..."));
		FMaterialValidationHelpers::FetchRevisionHistory(Collection.ModifiedObjects);
		FMaterialValidationHelpers::LoadDepotVersions(Collection.ModifiedObjects, Collection.ReplacementObjects, Collection.DepotPackages);

		if (SlowTask.ShouldCancel())
		{
			for (UPackage* DepotPackage : Collection.DepotPackages)
			{
				DepotPackage->MarkAsGarbage();
			}
			return FReply::Handled();
		}

		TMap<FSoftObjectPath, FPerMaterialResult> PerMaterialResults;
		{
			FScopedSlowTask MaterialTask(Collection.BaseMaterials.Num(), LOCTEXT("SlowShaderCount", "Computing shader counts..."));
			for (UMaterial* BaseMaterial : Collection.BaseMaterials)
			{
				MaterialTask.EnterProgressFrame(1.f, FText::FromName(BaseMaterial->GetFName()));
				if (MaterialTask.ShouldCancel())
				{
					for (UPackage* DepotPackage : Collection.DepotPackages)
					{
						DepotPackage->MarkAsGarbage();
					}
					return FReply::Handled();
				}
				FSoftObjectPath const Path(BaseMaterial);
				if (UMaterialValidationGroup* Group = FindGroupForMaterial(Groups, BaseMaterial))
				{
					FMaterialDatabaseAssetHierarchyInfo HierarchyInfo;
					UMaterialValidationLibrary::GetMaterialHierarchyInfo(Group, Path, HierarchyInfo);
					const int32 InstanceCount = FMath::Max(0, HierarchyInfo.MaterialPaths.Num() - 1);
					const bool bForceLoadObjects = true;
					const int32 Before = UMaterialValidationLibrary::GetModifiedShaderCount(Group, BaseMaterial, Collection.ModifiedObjects, Collection.ReplacementObjects, bForceLoadObjects);
					const int32 After  = UMaterialValidationLibrary::GetModifiedShaderCount(Group, BaseMaterial, Collection.ModifiedObjects, {}, bForceLoadObjects);
					const int32 ChangelistCount = Collection.ChangelistInstanceCounts.FindRef(Path);
					PerMaterialResults.Add(Path, {InstanceCount, ChangelistCount, After - Before});
				}
			}
		}

		// Need to explicitly release depot packages after work is done.
		for (UPackage* DepotPackage : Collection.DepotPackages)
		{
			DepotPackage->MarkAsGarbage();
		}

		UpdateResults(Collection, PerMaterialResults, EValidationMode::Slow);
		return FReply::Handled();
	}

	/** Returns true if the changelist contains at least one opened file whose asset class is a material-related type. */
	static bool ChangelistHasMaterialAsset(FSourceControlChangelistStateRef const& State)
	{
		IAssetRegistry* Registry = IAssetRegistry::Get();
		for (FSourceControlStateRef const& FileState : State->GetFilesStates())
		{
			FString PackageName;
			if (!FPackageName::TryConvertFilenameToLongPackageName(FileState->GetFilename(), PackageName))
			{
				continue;
			}

			TArray<FAssetData> Assets;
			Registry->GetAssetsByPackageName(*PackageName, Assets);
			for (FAssetData const& Asset : Assets)
			{
				if (Asset.IsInstanceOf(UMaterial::StaticClass())
					|| Asset.IsInstanceOf(UMaterialInstanceConstant::StaticClass())
					|| Asset.IsInstanceOf(UMaterialFunction::StaticClass())
					|| Asset.IsInstanceOf(UMaterialFunctionInstance::StaticClass())
					|| Asset.IsInstanceOf(UMaterialFunctionMaterialLayer::StaticClass()))
				{
					return true;
				}
			}
		}
		return false;
	}

	/** Combo box for choosing which changelist to validate. */
	TSharedPtr<SComboBox<TSharedPtr<FChangelistEntry>>> ChangelistComboBox;
	/** Changelists currently shown in the combo box after filtering. Repopulated on Refresh. */
	TArray<TSharedPtr<FChangelistEntry>> ChangelistEntries;
	/** Changelist currently selected in the combo box. */
	TSharedPtr<FChangelistEntry> SelectedEntry;
	/** If true, only changelists with at least one opened (non-shelved) file are shown. */
	bool bFilterOpenedFilesOnly = true;
	/** If true, only changelists containing material-related assets are shown. */
	bool bFilterMaterialAssetsOnly = false;
	/** Rows shown in the base-material list view. The last entry is always the synthetic summary row (added by UpdateResults). */
	TArray<TSharedPtr<FBaseMaterialRow>> BaseMaterialRows;
	/** The list view widget for the base-material table. */
	TSharedPtr<SListView<TSharedPtr<FBaseMaterialRow>>> BaseMaterialsList;
	/** Displays notes, warnings, and error messages below the table. */
	TSharedPtr<STextBlock> NotesText;
};

TSharedRef<SWidget> CreateChangelistTab()
{
	return SNew(SChangelistsTab);
}

} // namespace MaterialValidation

#undef LOCTEXT_NAMESPACE
