// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Widgets/SCompoundWidget.h"

namespace MaterialValidation
{
	/** Creates and returns the Changelists tab widget for the MaterialValidationToolkit. */
	TSharedRef<SWidget> CreateChangelistTab();
}
