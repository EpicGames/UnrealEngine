// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialValidationCommandlet.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Materials/MaterialInstance.h"
#include "MaterialValidationGroup.h"
#include "MaterialValidationLibrary.h"
#include "MaterialValidationModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialValidationCommandlet)

UMaterialValidationCommandlet::UMaterialValidationCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UMaterialValidationCommandlet::InitializePackageNames(const TArray<FString>& Tokens, TArray<FString>& MapPathNames, bool& bExplicitPackages)
{
	// Extract command line switches.
	for (FString const& Switch : Switches)
	{
		int32 Value;
		if (FParse::Value(*Switch, TEXT("UpdateMaterials="), Value))
		{
			bUpdateMaterials = Value != 0;
			UE_LOGF(LogMaterialValidation, Log, "UpdateMaterials: %d.", Value);
		}
		else if (FParse::Value(*Switch, TEXT("UpdatePermutations="), Value))
		{
			bUpdateMaterialPermutations = Value != 0;
			UE_LOGF(LogMaterialValidation, Log, "UpdatePermutations: %d.", Value);
		}
	}

	// When packages are not explicitly specified, default to the packages that contain the UMaterialValidationGroup objects in the project config.
	if (!bExplicitPackages)
	{
		TArray<UMaterialValidationGroup*> Groups;
		UMaterialValidationLibrary::GetAllGroups(Groups, /*bInSyncLoad*/true);

		for (UMaterialValidationGroup* Group : Groups)
		{
			if (UPackage* Package = Group->GetPackage())
			{
				MapPathNames.Add(Package->GetLoadedPath().GetLocalFullPath());
			
				UE_LOGF(LogMaterialValidation, Log, "Commandlet Adding Default Package: %ls.", *MapPathNames.Last());
			}
		}

		bExplicitPackages = true;
	}
}

bool UMaterialValidationCommandlet::ShouldSkipPackage(const FString& Filename)
{
	if (Super::ShouldSkipPackage(Filename))
	{
		return true;
	}

	// Skip package if we don't find any UMaterialValidationGroup objects in it.
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	FPackagePath PackagePath = FPackagePath::FromLocalPath(Filename);
	FName PackageName = PackagePath.GetPackageFName();

	TArray<FAssetData> PackageAssetDatas;
	if (!AssetRegistry.GetAssetsByPackageName(PackageName, PackageAssetDatas, /*bIncludeOnlyOnDiskAssets*/true))
	{
		return true;
	}

	bool bHasValidationGroupAsset = false;
	for (const FAssetData& AssetData : PackageAssetDatas)
	{
		if (AssetData.IsValid() && AssetData.IsInstanceOf(UMaterialValidationGroup::StaticClass()))
		{
			bHasValidationGroupAsset = true;
			break;
		}
	}

	return !bHasValidationGroupAsset;
}

void UMaterialValidationCommandlet::PerformAdditionalOperations(UObject* Object, bool& bSavePackage)
{
	UMaterialValidationGroup* Group = Cast<UMaterialValidationGroup>(Object);
	if (Group == nullptr)
	{
		return;
	}

	UE_LOGF(LogMaterialValidation, Display, "Commandlet Begin '%ls'", *Group->GetName());

	if (bUpdateMaterials)
	{
		UE_LOGF(LogMaterialValidation, Display, "Removing Invalid Materials.");
		UMaterialValidationLibrary::RemoveInvalidMaterialsFromGroup(Group);

		UE_LOGF(LogMaterialValidation, Display, "Adding Missing Materials.");
		UMaterialValidationLibrary::AddMissingMaterialsToGroup(Group);
	}

	if (bUpdateMaterialPermutations)
	{
		UE_LOGF(LogMaterialValidation, Display, "Updating Material Permutations.");
		UMaterialValidationLibrary::UpdateMaterialPermutationsInGroup(Group);
	}

	bSavePackage = bSavePackage || bUpdateMaterials || bUpdateMaterialPermutations;

	UE_LOGF(LogMaterialValidation, Display, "Commandlet End '%ls'", *Group->GetName());
}
