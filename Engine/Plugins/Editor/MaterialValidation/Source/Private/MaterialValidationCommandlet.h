// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Commandlets/BaseIteratePackagesCommandlet.h"
#include "MaterialValidationCommandlet.generated.h"

/** Commandlet to validate and update the UMaterialValidationGroup assets. */
UCLASS()
class UMaterialValidationCommandlet : public UBaseIteratePackagesCommandlet
{
	GENERATED_UCLASS_BODY()

	//~ Begin UBaseIteratePackagesCommandlet Interface
	virtual void InitializePackageNames(const TArray<FString>& Tokens, TArray<FString>& MapPathNames, bool& bExplicitPackages) override;
	virtual bool ShouldSkipPackage(const FString& Filename) override;
	virtual void PerformAdditionalOperations(class UObject* Object, bool& bSavePackage) override;
	//~ End UBaseIteratePackagesCommandlet Interface

private:
	/** true to update the materials bu removing those not found on disk and adding any materials found in the search paths. Disabled with UpdateMaterials=0 on commandline. */
	bool bUpdateMaterials = true;
	/** true to search for all material instances and add them to the approved list. Disabled with UpdatePermutations=0 on commandline. */
	bool bUpdateMaterialPermutations = true;
};
