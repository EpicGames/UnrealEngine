// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialValidationConfig.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialValidationConfig)

UMaterialValidationConfig::UMaterialValidationConfig(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

TSharedRef<IDetailCustomization> FDataValidationSettingsCustomization::MakeInstance()
{
	return MakeShareable(new FDataValidationSettingsCustomization);
}

void FDataValidationSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// Default behavior adds our properties _before_ the base ones, which isn't good for readability.
	// So break into two categories, and use category order to place our properties last.
	IDetailCategoryBuilder& BaseCategory = DetailBuilder.EditCategory(TEXT("Data Validation"));
	BaseCategory.SetSortOrder(0);
	IDetailCategoryBuilder& ExtendedCategory = DetailBuilder.EditCategory(TEXT("Data Validation Extensions"));
	ExtendedCategory.SetSortOrder(1);

	UObject* Config = GetMutableDefault<UMaterialValidationConfig>();
	FAddPropertyParams AddPropertyParams;
	ExtendedCategory.AddExternalObjects({ Config }, EPropertyLocation::Default, AddPropertyParams.HideRootObjectNode(true));
}
