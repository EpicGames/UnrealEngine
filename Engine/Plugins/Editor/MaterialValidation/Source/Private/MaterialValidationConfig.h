// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "IDetailCustomization.h"

#include "MaterialValidationConfig.generated.h"

class UMaterialValidationGroup;

/** Configuration for the material validation system. */
UCLASS(config = Editor, defaultconfig, meta = (DisplayName = "Data Validation Settings"), MinimalAPI)
class UMaterialValidationConfig : public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()

	// Disable auto registration of config UI becuase we use a specific detail customization below.
	virtual bool SupportsAutoRegistration() const override { return false; }

public:	
	/** Set true to enable validation. */
	UPROPERTY(config, EditAnywhere, Category = MaterialPermutationBudget, meta = (DisplayName = "Enable Validation"))
	bool bEnable = false;

	/** Threshold for shader count increase before showing to user. */
	UPROPERTY(config, EditAnywhere, Category = MaterialPermutationBudget, meta = (EditCondition = "bEnable"))
	int32 ShaderBudgetShowThreshold = 0;

	/** Threshold for shader count increase before triggering further submit approval. */
	UPROPERTY(config, EditAnywhere, Category = MaterialPermutationBudget, meta = (EditCondition = "bEnable"))
	int32 ShaderBudgetApprovalThreshold = 0;

	/** An array of Material Validation Groups to use during validation. */
	UPROPERTY(config, EditAnywhere, Category = MaterialPermutationBudget, meta = (EditCondition = "bEnable", DisplayName = "Material Validation Groups"))
	TArray<TSoftObjectPtr<UMaterialValidationGroup>> Groups;
};

/** 
 * Detail customization which adds the UMaterialValidationConfig properties to the core UDataValidationSettings config panel. 
 * This simplifies the UI by keeping related items in a sinlge place.
 */
class FDataValidationSettingsCustomization : public IDetailCustomization
{
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

public:
	static TSharedRef<IDetailCustomization> MakeInstance();
};
