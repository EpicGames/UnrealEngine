// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialValidationGroup.h"

#include "MaterialValidationLibrary.h"
#include "Misc/PackageName.h"
#include "UObject/DevObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialValidationGroup)

/** Custom serialization version for MaterialValidationGroup. */
struct FMaterialValidationGroupObjectVersion
{
	enum Type
	{
		// Before any version changes were made.
		BeforeCustomVersionWasAdded = 0,
		// Added base material overrides into permutation hash calculation.
		UpdatedPermutationHash,
		// Refined base material overrides to those that are relevant for shader permutations.
		UpdatedPermutationHash2,
		// Added asset data to give more accurate shader count reporting.
		AddedAssetData,

		// New versions can be added above this line.
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	const static FGuid GUID;

	FMaterialValidationGroupObjectVersion() = delete;
};

const FGuid FMaterialValidationGroupObjectVersion::GUID(0x0D03FD52, 0xBF174EAE, 0x89C8DE2A, 0x98BF53FE);
static FDevVersionRegistration GRegisterAtomFrameworkObjectVersion(FMaterialValidationGroupObjectVersion::GUID, FMaterialValidationGroupObjectVersion::LatestVersion, TEXT("MaterialValidationGroup"));

FStaticPermutationProperties::FStaticPermutationProperties()
	: TwoSided(0)
	, bIsThinSurface(0)
	, DitheredLODTransition(0)
	, bCastDynamicShadowAsMasked(0)
	, bOutputTranslucentVelocity(0)
	, bHasPixelAnimation(0)
	, bEnableTessellation(0)
	, BlendMode(BLEND_Opaque)
	, ShadingModel(MSM_DefaultLit)
	, OpacityMaskClipValue(.333333f)
	, UsageFlags(0)
{
}

uint32 GetTypeHash(const FStaticPermutationProperties& Arg)
{
	const uint8 PackedBools = 
		(Arg.TwoSided ? 1 << 0 : 0) |
		(Arg.bIsThinSurface ? 1 << 1 : 0) |
		(Arg.DitheredLODTransition ? 1 << 2 : 0) |
		(Arg.bCastDynamicShadowAsMasked ? 1 << 3 : 0) |
		(Arg.bOutputTranslucentVelocity ? 1 << 4 : 0) |
		(Arg.bHasPixelAnimation ? 1 << 5 : 0) |
		(Arg.bEnableTessellation ? 1 << 6 : 0);

	uint32 Hash = ::GetTypeHash(PackedBools);
	Hash = HashCombineFast(Hash, ::GetTypeHash(Arg.OpacityMaskClipValue));
	Hash = HashCombineFast(Hash, ::GetTypeHash(Arg.BlendMode));
	Hash = HashCombineFast(Hash, ::GetTypeHash(Arg.ShadingModel));
	Hash = HashCombineFast(Hash, ::GetTypeHash(Arg.UsageFlags));
	return Hash;
}

FStaticPermutationPropertyOverrideFlags::FStaticPermutationPropertyOverrideFlags()
	: bOverride_OpacityMaskClipValue(0)
	, bOverride_BlendMode(0)
	, bOverride_ShadingModel(0)
	, bOverride_DitheredLODTransition(0)
	, bOverride_CastDynamicShadowAsMasked(0)
	, bOverride_TwoSided(0)
	, bOverride_bIsThinSurface(0)
	, bOverride_OutputTranslucentVelocity(0)
	, bOverride_bHasPixelAnimation(0)
	, bOverride_bEnableTessellation(0)
	, Override_UsageFlags(0)
{
}

void UMaterialValidationGroup::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(FMaterialValidationGroupObjectVersion::GUID);

	if (Ar.IsLoading() && Ar.CustomVer(FMaterialValidationGroupObjectVersion::GUID) < FMaterialValidationGroupObjectVersion::AddedAssetData)
	{
		// Clear invalidated data.
		Materials.Reset();
	}
}

// Path picker returns absolute paths. Convert to package paths such as /Game/...
static void ConvertPaths(TArrayView<FDirectoryPath> Paths)
{
	for (FDirectoryPath& Path : Paths)
	{
		FString PackagePath;
		if (FPackageName::TryConvertFilenameToLongPackageName(Path.Path, PackagePath))
		{
			Path.Path = PackagePath;
		}
	}
}

void UMaterialValidationGroup::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property && PropertyChangedEvent.MemberProperty)
	{
		if (PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UMaterialValidationGroup, MaterialPaths))
		{
			ConvertPaths(MaterialPaths);
		}
		else if (PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UMaterialValidationGroup, MaterialExcludePaths))
		{
			ConvertPaths(MaterialExcludePaths);
		}
	}
}

void UMaterialValidationGroup::GetMaterialPaths(TArray<FSoftObjectPath>& OutMaterialPaths) const
{
	OutMaterialPaths.Empty(Materials.Num());
	for (TPair<FString, FMaterialValidationDesc> const& It : Materials)
	{
		OutMaterialPaths.Add(UMaterialValidationLibrary::ResolveAssetPath(It.Key));
	}
}

void UMaterialValidationGroup::UpdateMaterials()
{
	UMaterialValidationLibrary::RemoveInvalidMaterialsFromGroup(this);
	UMaterialValidationLibrary::AddMissingMaterialsToGroup(this);
}

void UMaterialValidationGroup::UpdatePermutations()
{
	UMaterialValidationLibrary::UpdateMaterialPermutationsInGroup(this);
}

/** Return a hash built for a material or material instance which will match for all instances that generate the same shader permutations. */
uint32 MaterialValidation::BuildPermutationHash(FStaticPermutationProperties const& InStaticProperties, TConstArrayView<uint32> InStaticSwitchValues, TConstArrayView<uint32> InComponentMaskValues, uint32 InMaterialLayerHash)
{
	uint32 Hash = GetTypeHash(InStaticProperties);
	for (uint32 Value : InStaticSwitchValues)
	{
		Hash = HashCombineFast(Hash, GetTypeHash(Value));
	}
	for (uint32 Value : InComponentMaskValues)
	{
		Hash = HashCombineFast(Hash, GetTypeHash(Value));
	}
	Hash = HashCombineFast(Hash, InMaterialLayerHash);
	return Hash;
}

