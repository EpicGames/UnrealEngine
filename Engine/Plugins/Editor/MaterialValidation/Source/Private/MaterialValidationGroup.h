// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataAsset.h"
#include "MaterialValidationAssetData.h"
#include "MaterialValidationGroup.generated.h"

/** Description of validation setup for a single UMaterial. */
USTRUCT(MinimalAPI)
struct FMaterialValidationDesc
{
	GENERATED_BODY()
	
	/** The asset data for the material. */
	UPROPERTY()
	FMaterialValidationAssetData_Material AssetData;

	/** The non-default asset datas from all of the material instances in the material hierarchy. */
	UPROPERTY()
	TArray<FMaterialValidationAssetData_MaterialInstance> MaterialInstanceAssetDatas;

	/** 
	 * Map of UMaterialInstance asset paths to MaterialInstanceAssetDatas array index. Value of -1 indicates a default AssetData.
	 * Paths are FString to avoid references to the actual assets which would need managing on asset delete etc. 
	 */
	UPROPERTY()
	TMap<FString, int32> MaterialInstances;

	/** Array of material permutations not including the one from the base material which is stored in AssetData.PermutationHash. */
	UPROPERTY()
	TArray<uint32> PermutationHashes;

	/** Number of shader types on the base material permutation. */
	UPROPERTY()
	int32 NumShadersBase = 0;

	/** Total sum of shader types for all permutations found of this material. */
	UPROPERTY(VisibleAnywhere, Category = Material)
	int32 NumShadersTotal = 0;
};

/** Data Asset containing the validation setup for a group of UMaterials. */
UCLASS(MinimalAPI, BlueprintType)
class UMaterialValidationGroup : public UDataAsset
{
	GENERATED_BODY()

public:
	/** Array of explicit directory paths that we use when searching for UMaterial packages that need validating. */
	UPROPERTY(EditAnywhere, Category = Material)
	TArray<FDirectoryPath> MaterialPaths;

	/** Array of explicit directory paths that we exclude when searching for UMaterial packages that need validating. */
	UPROPERTY(EditAnywhere, Category = Material)
	TArray<FDirectoryPath> MaterialExcludePaths;

	/** Use the project content path in addition to MaterialPaths when searching for UMaterials. */
	UPROPERTY(EditAnywhere, Category = Material, meta = (DisplayName = "Include All Project Content"))
	bool bAddProjectPath;

	/** Use all project plugin content paths in addition to MaterialPaths when searching for UMaterials. */
	UPROPERTY(EditAnywhere, Category = Material, meta = (DisplayName = "Include All Project Plugin Content"))
	bool bAddProjectPluginPaths;

	/** Map of UMaterial asset paths to validation descriptions. Paths are FString to avoid references to the actual assets which would need managing on asset delete etc. */
	UPROPERTY()
	TMap<FString, FMaterialValidationDesc> Materials;

	/** Get UMaterial assets resolved to FSoftObjectPaths. */
	UFUNCTION(BlueprintPure, Category = Material)
	void GetMaterialPaths(TArray<FSoftObjectPath>& OutMaterialPaths) const;

	/** 
	 * Add all Material assets found under the material search paths.
	 * The new material assets will have empty permutation lists.
	 */
	UFUNCTION(CallInEditor, Category = Actions)
	void UpdateMaterials();

	/** 
	 * Update current state of all Materials.
	 * For each material this will traverse the material instance hierarchy and add all existing permutations to the permutation list.
	 */
	UFUNCTION(CallInEditor, Category = Actions)
	void UpdatePermutations();

protected:
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
};
