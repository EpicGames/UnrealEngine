// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialValidationGroupDiff.h"

#include "AssetRegistry/IAssetRegistry.h"
#include "Editor.h"
#include "Framework/Application/SlateApplication.h"
#include "MaterialValidationGroup.h"
#include "MaterialValidationLibrary.h"
#include "MaterialValidationLibraryTypes.h"
#include "MaterialValidationToolkitShared.h"
#include "Misc/TextFilterExpressionEvaluator.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"

#define LOCTEXT_NAMESPACE "MaterialValidation"

namespace MaterialValidation {

/** Per-material row data for the material diff table. Holds the counts and deltas from both snapshots. */
struct FMaterialDiffStats
{
	FSoftObjectPath AssetPath;
	EMaterialInstanceDiffType ChangeType = EMaterialInstanceDiffType::Modified;
	
	int32 OldInstanceCount = 0;
	int32 NewInstanceCount = 0;
	int32 InstanceDelta = 0;
	int32 OldShaderCount = 0;
	int32 NewShaderCount = 0;
	int32 ShaderDelta = 0;
	int32 OldPermutationCount = 0;
	int32 NewPermutationCount = 0;
	int32 PermutationDelta = 0;

	static const FName ColumnId_AssetPath;
	static const FName ColumnId_OldInstanceCount;
	static const FName ColumnId_NewInstanceCount;
	static const FName ColumnId_InstanceDelta;
	static const FName ColumnId_OldShaderCount;
	static const FName ColumnId_NewShaderCount;
	static const FName ColumnId_ShaderDelta;
	static const FName ColumnId_OldPermutationCount;
	static const FName ColumnId_NewPermutationCount;
	static const FName ColumnId_PermutationDelta;
};

const FName FMaterialDiffStats::ColumnId_AssetPath("CID_Diff_AssetPath");
const FName FMaterialDiffStats::ColumnId_OldInstanceCount("CID_Diff_OldInstanceCount");
const FName FMaterialDiffStats::ColumnId_NewInstanceCount("CID_Diff_NewInstanceCount");
const FName FMaterialDiffStats::ColumnId_InstanceDelta("CID_Diff_InstanceDelta");
const FName FMaterialDiffStats::ColumnId_OldShaderCount("CID_Diff_OldShaderCount");
const FName FMaterialDiffStats::ColumnId_NewShaderCount("CID_Diff_NewShaderCount");
const FName FMaterialDiffStats::ColumnId_ShaderDelta("CID_Diff_ShaderDelta");
const FName FMaterialDiffStats::ColumnId_OldPermutationCount("CID_Diff_OldPermCount");
const FName FMaterialDiffStats::ColumnId_NewPermutationCount("CID_Diff_NewPermCount");
const FName FMaterialDiffStats::ColumnId_PermutationDelta("CID_Diff_PermDelta");

/** Build diff rows by comparing two materials groups. */
static TArray<TSharedPtr<FMaterialDiffStats>> BuildDiffRows(UMaterialValidationGroup const& InOldGroup, UMaterialValidationGroup const& InNewGroup)
{
	TArray<TSharedPtr<FMaterialDiffStats>> Rows;

	TSet<FString> AllPaths;
	for (auto const& Pair : InOldGroup.Materials) 
	{
		AllPaths.Add(Pair.Key); 
	}
	for (auto const& Pair : InNewGroup.Materials) 
	{
		AllPaths.Add(Pair.Key); 
	}

	for (FString const& PathStr : AllPaths)
	{
		FMaterialValidationDesc const* OldDesc = InOldGroup.Materials.Find(PathStr);
		FMaterialValidationDesc const* NewDesc = InNewGroup.Materials.Find(PathStr);

		TSharedPtr<FMaterialDiffStats> Row = MakeShared<FMaterialDiffStats>();
		Row->AssetPath = UMaterialValidationLibrary::ResolveAssetPath(PathStr);

		if (OldDesc == nullptr)
		{
			Row->ChangeType = EMaterialInstanceDiffType::Added;
			Row->NewInstanceCount = NewDesc->MaterialInstances.Num();
			Row->NewShaderCount = NewDesc->NumShadersTotal;
			Row->NewPermutationCount = NewDesc->PermutationHashes.Num() + 1;
			Row->ShaderDelta = Row->NewShaderCount;
			Row->InstanceDelta = Row->NewInstanceCount;
			Row->PermutationDelta = Row->NewPermutationCount;
		}
		else if (NewDesc == nullptr)
		{
			Row->ChangeType = EMaterialInstanceDiffType::Removed;
			Row->OldInstanceCount = OldDesc->MaterialInstances.Num();
			Row->OldShaderCount = OldDesc->NumShadersTotal;
			Row->OldPermutationCount = OldDesc->PermutationHashes.Num() + 1;
			Row->ShaderDelta = -Row->OldShaderCount;
			Row->InstanceDelta = -Row->OldInstanceCount;
			Row->PermutationDelta = -Row->OldPermutationCount;
		}
		else
		{
			Row->ChangeType = EMaterialInstanceDiffType::Modified;
			Row->OldInstanceCount = OldDesc->MaterialInstances.Num();
			Row->NewInstanceCount = NewDesc->MaterialInstances.Num();
			Row->OldShaderCount = OldDesc->NumShadersTotal;
			Row->NewShaderCount = NewDesc->NumShadersTotal;
			Row->OldPermutationCount = OldDesc->PermutationHashes.Num() + 1;
			Row->NewPermutationCount = NewDesc->PermutationHashes.Num() + 1;
			Row->InstanceDelta = Row->NewInstanceCount - Row->OldInstanceCount;
			Row->ShaderDelta = Row->NewShaderCount - Row->OldShaderCount;
			Row->PermutationDelta = Row->NewPermutationCount - Row->OldPermutationCount;

			// Skip unchanged materials.
			if (Row->InstanceDelta == 0 && Row->ShaderDelta == 0 && Row->PermutationDelta == 0)
			{
				continue;
			}
		}

		Rows.Add(MoveTemp(Row));
	}

	return Rows;
}

/** Table row widget for one material in the material diff table. */
class SMaterialDiffTableRow : public SMultiColumnTableRow<TSharedPtr<FMaterialDiffStats>>
{
	TSharedPtr<FMaterialDiffStats> RowItem;

public:
	SLATE_BEGIN_ARGS(SMaterialDiffTableRow) {}
		SLATE_ARGUMENT(TSharedPtr<FMaterialDiffStats>, Item)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable)
	{
		RowItem = InArgs._Item;
		SMultiColumnTableRow<TSharedPtr<FMaterialDiffStats>>::Construct(FSuperRowType::FArguments(), InOwnerTable);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (!RowItem.IsValid())
		{
			return SNew(STextBlock).Text(GUnknownText);
		}

		const bool bAdded = RowItem->ChangeType == EMaterialInstanceDiffType::Added;
		const bool bHasOld = !bAdded;
		const bool bRemoved = RowItem->ChangeType == EMaterialInstanceDiffType::Removed;
		const bool bHasNew = !bRemoved;

		// Helper to generate a numeric cell.
		auto MakeNumericCell = [](int32 InValue, bool bShow) -> TSharedRef<SWidget>
		{
			return SNew(SBox).HAlign(HAlign_Center).VAlign(VAlign_Center)
			[
				SNew(STextBlock).Text(bShow ? FText::AsNumber(InValue) : GEmptyText)
			];
		};

		// Helper to generate a delta cell.
		auto MakeDeltaCell = [](int32 InDelta, bool bBothSidesExist) -> TSharedRef<SWidget>
		{
			const FText DeltaText = bBothSidesExist ? GetTextSignedInt(InDelta) : GEmptyText;
			const FSlateColor Color = bBothSidesExist ? GetDeltaColor(InDelta) : GetDeltaColor(0);
			return SNew(SBox).HAlign(HAlign_Center).VAlign(VAlign_Center)
			[
				SNew(STextBlock).Text(DeltaText).ColorAndOpacity(Color)
			];
		};

		if (ColumnName == FMaterialDiffStats::ColumnId_AssetPath)
		{
			const FSlateColor PathColor = bAdded ? GetDeltaColor(1) : bRemoved ? GetDeltaColor(-1) : GetDeltaColor(0);
			return MaterialValidation::GenerateAssetPathWidget(RowItem->AssetPath, PathColor);
		}
		if (ColumnName == FMaterialDiffStats::ColumnId_OldInstanceCount) { return MakeNumericCell(RowItem->OldInstanceCount, bHasOld); }
		if (ColumnName == FMaterialDiffStats::ColumnId_NewInstanceCount) { return MakeNumericCell(RowItem->NewInstanceCount, bHasNew); }
		if (ColumnName == FMaterialDiffStats::ColumnId_InstanceDelta) { return MakeDeltaCell(RowItem->InstanceDelta, bHasOld && bHasNew); }
		if (ColumnName == FMaterialDiffStats::ColumnId_OldShaderCount) { return MakeNumericCell(RowItem->OldShaderCount, bHasOld); }
		if (ColumnName == FMaterialDiffStats::ColumnId_NewShaderCount) { return MakeNumericCell(RowItem->NewShaderCount, bHasNew); }
		if (ColumnName == FMaterialDiffStats::ColumnId_ShaderDelta) { return MakeDeltaCell(RowItem->ShaderDelta, bHasOld && bHasNew); }
		if (ColumnName == FMaterialDiffStats::ColumnId_OldPermutationCount) { return MakeNumericCell(RowItem->OldPermutationCount, bHasOld); }
		if (ColumnName == FMaterialDiffStats::ColumnId_NewPermutationCount) { return MakeNumericCell(RowItem->NewPermutationCount, bHasNew); }
		if (ColumnName == FMaterialDiffStats::ColumnId_PermutationDelta) { return MakeDeltaCell(RowItem->PermutationDelta, bHasOld && bHasNew); }

		return SNew(STextBlock).Text(GUnknownText);
	}
};

/** Delegate for notification of material data row selection. */
DECLARE_DELEGATE_OneParam(FOnMaterialDiffRowSelected, TSharedPtr<FMaterialDiffStats>);

/**
 * Widget for the material diff table. 
 * Shows one sortable/filterable row for each base material that changed between the two group snapshots.
 * Fires FOnMaterialDiffRowSelected when the user clicks a row to populate the material instance tree.
 */
class SMaterialDiffTable : public SCompoundWidget
{
	/** All the available rows. */
	TArray<TSharedPtr<FMaterialDiffStats>> AllRows;
	/** An ordered and filtered subset of AllRows. */
	TArray<TSharedPtr<FMaterialDiffStats>> FilteredRows;
	/** A slate view of the table. */
	TSharedPtr<SListView<TSharedPtr<FMaterialDiffStats>>> ListView;

	/** Search text used for filtering the rows. */
	FString SearchText;
	/** Selected column name for sorting the table. */
	FName SortColumnId = NAME_None;
	/** Selected mode for sorting the table. */
	EColumnSortMode::Type SortMode = EColumnSortMode::None;

	/** Delegate for row selection. */
	FOnMaterialDiffRowSelected OnRowSelected;

public:
	SLATE_BEGIN_ARGS(SMaterialDiffTable) {}
		SLATE_ARGUMENT(TArray<TSharedPtr<FMaterialDiffStats>>, InitialRows)
		SLATE_EVENT(FOnMaterialDiffRowSelected, OnRowSelected)
	SLATE_END_ARGS()

	/** Builds the slate table view. */
	void Construct(const FArguments& InArgs)
	{
		AllRows = InArgs._InitialRows;
		FilteredRows = InArgs._InitialRows;
		OnRowSelected = InArgs._OnRowSelected;

		SortColumnId = FMaterialDiffStats::ColumnId_ShaderDelta;
		SortMode = EColumnSortMode::Descending;
		UpdateSort();

		ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot().AutoHeight().Padding(2)
			[
				SNew(SSearchBox)
					.HintText(LOCTEXT("DiffSearchHint", "Filter By Asset"))
					.OnTextChanged(this, &SMaterialDiffTable::OnSearchTextChanged)
			]
			+ SVerticalBox::Slot().FillHeight(1.f).Padding(2)
			[
				SAssignNew(ListView, SListView<TSharedPtr<FMaterialDiffStats>>)
				.HeaderRow(BuildHeaderRow())
				.ListItemsSource(&FilteredRows)
				.OnGenerateRow_Lambda([](TSharedPtr<FMaterialDiffStats> InItem, const TSharedRef<STableViewBase>& InOwnerTable)
				{
					return SNew(SMaterialDiffTableRow, InOwnerTable).Item(InItem);
				})
				.SelectionMode(ESelectionMode::Single)
				.OnSelectionChanged_Lambda([this](TSharedPtr<FMaterialDiffStats> InItem, ESelectInfo::Type)
				{
					OnRowSelected.ExecuteIfBound(InItem);
				})
			]
		];
	}

private:
	/** Builds the header row for the slate table view. */
	TSharedRef<SHeaderRow> BuildHeaderRow()
	{
		auto MakeSortableCol = [this](FName InId, FText InLabel) -> SHeaderRow::FColumn::FArguments
		{
			return SHeaderRow::Column(InId)
				.HAlignHeader(HAlign_Center)
				.HAlignCell(HAlign_Center)
				.DefaultLabel(InLabel)
				.OnSort(this, &SMaterialDiffTable::OnColumnSortModeChanged)
				.FixedWidth(140);
		};

		return SNew(SHeaderRow)
			+ SHeaderRow::Column(FMaterialDiffStats::ColumnId_AssetPath)
				.HAlignHeader(HAlign_Center).HAlignCell(HAlign_Left)
				.DefaultLabel(LOCTEXT("DiffAssetPath", "Asset Path"))
				.OnSort(this, &SMaterialDiffTable::OnColumnSortModeChanged)
				.ManualWidth(GAssetNameColumnWidth)
				.ManualWidth_Lambda([this]() 
				{
					return GAssetNameColumnWidth;
				})
				.OnWidthChanged_Lambda([this](float NewWidth) 
				{
					GAssetNameColumnWidth = NewWidth;
				})
			+ MakeSortableCol(FMaterialDiffStats::ColumnId_ShaderDelta, LOCTEXT("ShaderDelta", "Shaders (Delta)"))
			+ MakeSortableCol(FMaterialDiffStats::ColumnId_InstanceDelta, LOCTEXT("InstanceDelta", "Instances (Delta)"))
			+ MakeSortableCol(FMaterialDiffStats::ColumnId_PermutationDelta, LOCTEXT("PermDelta", "Permutations (Delta)"))
			+ MakeSortableCol(FMaterialDiffStats::ColumnId_OldShaderCount, LOCTEXT("OldShaders", "Shaders (Old)"))
			+ MakeSortableCol(FMaterialDiffStats::ColumnId_NewShaderCount, LOCTEXT("NewShaders", "Shaders (New)"))
			+ MakeSortableCol(FMaterialDiffStats::ColumnId_OldInstanceCount, LOCTEXT("OldInstances", "Instances (Old)"))
			+ MakeSortableCol(FMaterialDiffStats::ColumnId_NewInstanceCount, LOCTEXT("NewInstances", "Instances (New)"))
			+ MakeSortableCol(FMaterialDiffStats::ColumnId_OldPermutationCount, LOCTEXT("OldPermutations", "Permutations (Old)"))
			+ MakeSortableCol(FMaterialDiffStats::ColumnId_NewPermutationCount, LOCTEXT("NewPermutations", "Permutations (New)"));
	}

	/** Called when modifying the search text. */
	void OnSearchTextChanged(const FText& InText)
	{
		SearchText = InText.ToString();
		UpdateFilter();
		UpdateSort();
		ListView->RequestListRefresh();
	}

	/** Called when clicking on a column header to sort. */
	void OnColumnSortModeChanged(const EColumnSortPriority::Type, const FName& ColumnId, const EColumnSortMode::Type InSortMode)
	{
		if (SortColumnId != ColumnId)
		{
			SortMode = (ColumnId == FMaterialDiffStats::ColumnId_AssetPath) ? EColumnSortMode::Ascending : EColumnSortMode::Descending;
			SortColumnId = ColumnId;
		}
		else
		{
			SortMode = (SortMode == EColumnSortMode::Descending) ? EColumnSortMode::Ascending : EColumnSortMode::Descending;
		}
		UpdateSort();
		ListView->RequestListRefresh();
	}

	/**  Apply filter to AllRows to recreate FilteredRows. */
	void UpdateFilter()
	{
		FilteredRows.Reset();

		FTextFilterExpressionEvaluator Filter(ETextFilterExpressionEvaluatorMode::BasicString);
		Filter.SetFilterText(FText::FromString(SearchText));

		for (TSharedPtr<FMaterialDiffStats> const& Row : AllRows)
		{
			if (SearchText.IsEmpty() || Filter.TestTextFilter(FBasicStringFilterExpressionContext(Row->AssetPath.ToString())))
			{
				FilteredRows.Add(Row);
			}
		}
	}

	/**  Apply sort to FilteredRows. */
	void UpdateSort()
	{
		const bool bSortModifier = (SortMode == EColumnSortMode::Descending);

		if (SortColumnId == FMaterialDiffStats::ColumnId_AssetPath)
		{
			FilteredRows.Sort([bSortModifier](auto const& A, auto const& B)
			{ 
				return bSortModifier ? A->AssetPath.LexicalLess(B->AssetPath) : B->AssetPath.LexicalLess(A->AssetPath); 
			});
		}
		else
		{
			// For delta columns sort by signed value; for raw count columns sort by value.
			auto GetSortKey = [](TSharedPtr<FMaterialDiffStats> const& R, FName ColumnId) -> int32
			{
				if (ColumnId == FMaterialDiffStats::ColumnId_ShaderDelta) { return R->ShaderDelta; }
				if (ColumnId == FMaterialDiffStats::ColumnId_InstanceDelta) { return R->InstanceDelta; }
				if (ColumnId == FMaterialDiffStats::ColumnId_PermutationDelta) { return R->PermutationDelta; }
				if (ColumnId == FMaterialDiffStats::ColumnId_OldShaderCount) { return R->OldShaderCount; }
				if (ColumnId == FMaterialDiffStats::ColumnId_NewShaderCount) { return R->NewShaderCount; }
				if (ColumnId == FMaterialDiffStats::ColumnId_OldInstanceCount) { return R->OldInstanceCount; }
				if (ColumnId == FMaterialDiffStats::ColumnId_NewInstanceCount) { return R->NewInstanceCount; }
				if (ColumnId == FMaterialDiffStats::ColumnId_OldPermutationCount) { return R->OldPermutationCount; }
				if (ColumnId == FMaterialDiffStats::ColumnId_NewPermutationCount) { return R->NewPermutationCount; }
				return 0;
			};

			// Returns true when the cell for this row/column displays "-" (the value is not applicable).
			// Added rows have no Old data; Removed rows have no New data; both show "-" in delta columns.
			// Rows with blank cells should always sort last.
			auto IsBlankCell = [](TSharedPtr<FMaterialDiffStats> const& R, FName ColumnId) -> bool
			{
				const bool bAdded   = R->ChangeType == EMaterialInstanceDiffType::Added;
				const bool bRemoved = R->ChangeType == EMaterialInstanceDiffType::Removed;

				if (!bAdded && !bRemoved)
				{
					return false;
				}
				if (ColumnId == FMaterialDiffStats::ColumnId_ShaderDelta
					|| ColumnId == FMaterialDiffStats::ColumnId_InstanceDelta
					|| ColumnId == FMaterialDiffStats::ColumnId_PermutationDelta)
				{
					return true;
				}
				if (bAdded)
				{
					return ColumnId == FMaterialDiffStats::ColumnId_OldShaderCount
						|| ColumnId == FMaterialDiffStats::ColumnId_OldInstanceCount
						|| ColumnId == FMaterialDiffStats::ColumnId_OldPermutationCount;
				}
				return ColumnId == FMaterialDiffStats::ColumnId_NewShaderCount
					|| ColumnId == FMaterialDiffStats::ColumnId_NewInstanceCount
					|| ColumnId == FMaterialDiffStats::ColumnId_NewPermutationCount;
			};

			const FName ColId = SortColumnId;
			FilteredRows.Sort([bSortModifier, ColId, &GetSortKey, &IsBlankCell](auto const& A, auto const& B)
			{
				const bool bABlank = IsBlankCell(A, ColId);
				const bool bBBlank = IsBlankCell(B, ColId);
				if (bABlank != bBBlank)
				{
					// Blank cells always sort after real values, regardless of sort direction.
					return bBBlank;
				}
				if (bABlank && bBBlank)
				{
					// Both cells are blank — use asset path as a stable secondary sort.
					return A->AssetPath.LexicalLess(B->AssetPath);
				}
				return bSortModifier ? GetSortKey(A, ColId) > GetSortKey(B, ColId) : GetSortKey(A, ColId) < GetSortKey(B, ColId);
			});
		}
	}
};

/** Node in the material instance property diff tree. */
struct FMaterialDiffTreeNode
{
	/** true if this is a property node. */
	bool bIsPropertyNode = false;
	/** Populated for material instance(parent) nodes. */
	FMaterialInstanceDiffResult DiffResult;
	/** Populated for property nodes. */
	FMaterialInstancePropertyDiff PropertyDiff;
	/** Children properties of this node. */
	TArray<TSharedPtr<FMaterialDiffTreeNode>> Children;
};

/** Table row for the material instance property diff tree.  */
class SMaterialInstanceDiffTreeRow : public SMultiColumnTableRow<TSharedPtr<FMaterialDiffTreeNode>>
{
	/** The node for the row. */
	TSharedPtr<FMaterialDiffTreeNode> Node;

public:
	static const FName ColAssetPath;
	static const FName ColProperty;
	static const FName ColOldValue;
	static const FName ColNewValue;

	SLATE_BEGIN_ARGS(SMaterialInstanceDiffTreeRow) {}
		SLATE_ARGUMENT(TSharedPtr<FMaterialDiffTreeNode>, Node)
	SLATE_END_ARGS()

	/** Builds the slate node view. */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		Node = InArgs._Node;
		SMultiColumnTableRow<TSharedPtr<FMaterialDiffTreeNode>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override
	{
		if (!Node.IsValid())
		{
			return SNullWidget::NullWidget;
		}

		// Property difference row.
		// Asset path column is empty and property/value columns are filled.
		if (Node->bIsPropertyNode)
		{
			if (InColumnName == ColProperty)
			{
				return SNew(SHorizontalBox)
					+ SHorizontalBox::Slot().FillWidth(1.f).HAlign(HAlign_Center).VAlign(VAlign_Center)
					[
						SNew(STextBlock).Text(FText::FromName(Node->PropertyDiff.PropertyName))
					];
			}
			if (InColumnName == ColOldValue)
			{
				return SNew(SBox).HAlign(HAlign_Center).VAlign(VAlign_Center)
				[
					SNew(STextBlock).Text(FText::FromString(Node->PropertyDiff.OldValue))
				];
			}
			if (InColumnName == ColNewValue)
			{
				return SNew(SBox).HAlign(HAlign_Center).VAlign(VAlign_Center)
				[
					SNew(STextBlock).Text(FText::FromString(Node->PropertyDiff.NewValue))
				];
			}
			return SNullWidget::NullWidget;
		}

		// Material instance (parent) row.
		// Show expander arrow and coloured asset path in the asset path column only.
		if (InColumnName == ColAssetPath)
		{
			FSlateColor NodeColor = GetDeltaColor(0);
			if (Node->DiffResult.DiffType == EMaterialInstanceDiffType::Added)
			{
				NodeColor = GetDeltaColor(1);
			}
			else if (Node->DiffResult.DiffType == EMaterialInstanceDiffType::Removed)
			{
				NodeColor = GetDeltaColor(-1);
			}
			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot().AutoWidth().VAlign(VAlign_Center)
				[
					SNew(SExpanderArrow, SharedThis(this)).IndentAmount(12)
				]
				+ SHorizontalBox::Slot().FillWidth(1.f)
				[
					MaterialValidation::GenerateAssetPathWidget(Node->DiffResult.InstancePath, NodeColor)
				];
		}

		return SNullWidget::NullWidget;
	}
};

const FName SMaterialInstanceDiffTreeRow::ColAssetPath = "AssetPath";
const FName SMaterialInstanceDiffTreeRow::ColProperty  = "Property";
const FName SMaterialInstanceDiffTreeRow::ColOldValue  = "OldValue";
const FName SMaterialInstanceDiffTreeRow::ColNewValue  = "NewValue";

/**
 * Material instance property diff tree. 
 * This is populated by GetMaterialValidationDescDiff when the user selects a row in SMaterialDiffTable.
 * It shows Added/Removed instances as leaf nodes and Modified instances as expandable nodes with property diff children.
 */
class SMaterialInstanceDiffTree : public SCompoundWidget
{
	/** The older group snapshot. Held as a weak pointer since this widget is not a UObject. */
	TWeakObjectPtr<const UMaterialValidationGroup> OldGroup;
	/** The newer group snapshot. Held as a weak pointer since this widget is not a UObject. */
	TWeakObjectPtr<const UMaterialValidationGroup> NewGroup;
	/** Path of the base material currently shown in the tree. */
	FSoftObjectPath SelectedMaterialPath;

	/** Full unfiltered list of parent (instance) nodes built from the last diff call. */
	TArray<TSharedPtr<FMaterialDiffTreeNode>> AllRootNodes;
	/** Subset of AllRootNodes that pass the current search filters. */
	TArray<TSharedPtr<FMaterialDiffTreeNode>> FilteredRootNodes;
	/** The tree view widget. */
	TSharedPtr<STreeView<TSharedPtr<FMaterialDiffTreeNode>>> TreeView;

	/** Current text from the instance path search box. */
	FString InstanceSearchText;
	/** Current text from the property name search box. */
	FString PropertySearchText;

	/** Tracks whether all tree nodes are currently expanded, toggled by clicking the Asset Path column header. */
	bool bAllExpanded = false;

public:
	SLATE_BEGIN_ARGS(SMaterialInstanceDiffTree) {}
		SLATE_ARGUMENT(UMaterialValidationGroup const*, OldGroup)
		SLATE_ARGUMENT(UMaterialValidationGroup const*, NewGroup)
	SLATE_END_ARGS()

	/** Builds the slate view. */
	void Construct(const FArguments& InArgs)
	{
		OldGroup = InArgs._OldGroup;
		NewGroup = InArgs._NewGroup;

		ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot().AutoHeight().Padding(2)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot().FillWidth(1.f).Padding(0, 0, 4, 0)
				[
					SNew(SSearchBox)
						.HintText(LOCTEXT("InstanceSearchHint", "Filter By Instance Name"))
						.OnTextChanged(this, &SMaterialInstanceDiffTree::OnInstanceSearchChanged)
				]
				+ SHorizontalBox::Slot().FillWidth(1.f)
				[
					SNew(SSearchBox)
						.HintText(LOCTEXT("PropertySearchHint", "Filter By Property Name"))
						.OnTextChanged(this, &SMaterialInstanceDiffTree::OnPropertySearchChanged)
				]
			]
			+ SVerticalBox::Slot().FillHeight(1.f).Padding(2)
			[
				SAssignNew(TreeView, STreeView<TSharedPtr<FMaterialDiffTreeNode>>)
					.SelectionMode(ESelectionMode::None)
					.TreeItemsSource(&FilteredRootNodes)
					.OnGenerateRow(this, &SMaterialInstanceDiffTree::OnGenerateRow)
					.OnGetChildren_Lambda([](TSharedPtr<FMaterialDiffTreeNode> InNode, TArray<TSharedPtr<FMaterialDiffTreeNode>>& OutChildren)
					{
						OutChildren = InNode->Children;
					})
					.HeaderRow(
						SNew(SHeaderRow)
						+ SHeaderRow::Column(SMaterialInstanceDiffTreeRow::ColAssetPath)
							.HAlignHeader(HAlign_Center)
							.ManualWidth(GAssetNameColumnWidth)
							.ManualWidth_Lambda([this]()
							{
								return GAssetNameColumnWidth;
							})
							.OnWidthChanged_Lambda([this](float NewWidth)
							{
								GAssetNameColumnWidth = NewWidth;
							})
							.HeaderContent()
							[
								SNew(SBox)
								.WidthOverride(TAttribute<FOptionalSize>::CreateLambda([]() -> FOptionalSize { return GAssetNameColumnWidth; }))
								[
									SNew(SButton)
									.ButtonStyle(FAppStyle::Get(), "NoBorder")
									.HAlign(HAlign_Center)
									.VAlign(VAlign_Center)
									.ToolTipText(LOCTEXT("AssetPathHeaderTooltip", "Click to expand or collapse all rows"))
									.OnClicked(this, &SMaterialInstanceDiffTree::OnToggleExpandAll)
									[
										SNew(STextBlock)
										.Text(LOCTEXT("ColAssetPath", "Asset Path"))
									]
								]
							]
						+ SHeaderRow::Column(SMaterialInstanceDiffTreeRow::ColProperty)
							.HAlignHeader(HAlign_Center)
							.HAlignCell(HAlign_Center)
							.DefaultLabel(LOCTEXT("ColProperty", "Property"))
							.FixedWidth(150.f)
						+ SHeaderRow::Column(SMaterialInstanceDiffTreeRow::ColOldValue)
							.HAlignHeader(HAlign_Center)
							.HAlignCell(HAlign_Center)
							.DefaultLabel(LOCTEXT("ColOldValue", "Old Value"))
							.FixedWidth(90.f)
						+ SHeaderRow::Column(SMaterialInstanceDiffTreeRow::ColNewValue)
							.HAlignHeader(HAlign_Center)
							.HAlignCell(HAlign_Center)
							.DefaultLabel(LOCTEXT("ColNewValue", "New Value"))
							.FixedWidth(90.f)
					)
			]
		];
	}

	/** Populate the tree for the given base material path. */
	void SetMaterial(FSoftObjectPath const& InBaseMaterialPath)
	{
		SelectedMaterialPath = InBaseMaterialPath;
		RebuildFromDiff();
	}

private:
	/** Call GetMaterialValidationDescDiff for the current material and rebuild AllRootNodes from the result. */
	void RebuildFromDiff()
	{
		AllRootNodes.Reset();

		if (!OldGroup.IsValid() || !NewGroup.IsValid() || SelectedMaterialPath.IsNull())
		{
			ApplyFilters();
			return;
		}

		TArray<FMaterialInstanceDiffResult> Diffs;
		UMaterialValidationLibrary::GetMaterialValidationDescDiff(OldGroup.Get(), NewGroup.Get(), SelectedMaterialPath, /*bAllowAssetLoad=*/true, Diffs);

		for (FMaterialInstanceDiffResult const& Diff : Diffs)
		{
			TSharedPtr<FMaterialDiffTreeNode> Node = MakeShared<FMaterialDiffTreeNode>();
			Node->DiffResult = Diff;

			if (Diff.DiffType == EMaterialInstanceDiffType::Modified)
			{
				for (FMaterialInstancePropertyDiff const& PropDiff : Diff.PropertyDiffs)
				{
					TSharedPtr<FMaterialDiffTreeNode> Child = MakeShared<FMaterialDiffTreeNode>();
					Child->bIsPropertyNode = true;
					Child->PropertyDiff = PropDiff;
					Node->Children.Add(Child);
				}
			}

			AllRootNodes.Add(MoveTemp(Node));
		}

		ApplyFilters();
	}

	/** Rebuild FilteredRootNodes from AllRootNodes by applying both the instance path and property name filters. */
	void ApplyFilters()
	{
		FilteredRootNodes.Reset();

		FTextFilterExpressionEvaluator AssetFilter(ETextFilterExpressionEvaluatorMode::BasicString);
		AssetFilter.SetFilterText(FText::FromString(InstanceSearchText));

		FTextFilterExpressionEvaluator PropertyFilter(ETextFilterExpressionEvaluatorMode::BasicString);
		PropertyFilter.SetFilterText(FText::FromString(PropertySearchText));

		for (TSharedPtr<FMaterialDiffTreeNode> const& Node : AllRootNodes)
		{
			// Instance path filter.
			if (!InstanceSearchText.IsEmpty() && 
				!AssetFilter.TestTextFilter(FBasicStringFilterExpressionContext(Node->DiffResult.InstancePath.ToString())))
			{
				continue;
			}

			// Property name filter.
			// Added/Removed nodes have no properties to match, so hide them when the filter is active. 
			// Modified nodes are hidden unless a property name matches.
			if (!PropertySearchText.IsEmpty())
			{
				if (Node->DiffResult.DiffType != EMaterialInstanceDiffType::Modified)
				{
					continue;
				}

				bool bHasAnyMatch = false;
				for (TSharedPtr<FMaterialDiffTreeNode> const& Child : Node->Children)
				{
					if (PropertyFilter.TestTextFilter(FBasicStringFilterExpressionContext(Child->PropertyDiff.PropertyName.ToString())))
					{
						bHasAnyMatch = true;
						break;
					}
				}
				if (!bHasAnyMatch)
				{
					continue;
				}
			}

			FilteredRootNodes.Add(Node);
		}

		if (TreeView.IsValid())
		{
			TreeView->RequestTreeRefresh();
		}
	}

	/** Generate a table row for either a parent instance node or a child property diff node. */
	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FMaterialDiffTreeNode> InNode, const TSharedRef<STableViewBase>& InOwnerTable)
	{
		return SNew(SMaterialInstanceDiffTreeRow, InOwnerTable).Node(InNode);
	}

	/** Called when the instance path search box text changes. */
	void OnInstanceSearchChanged(const FText& InText)
	{
		InstanceSearchText = InText.ToString();
		ApplyFilters();
	}

	/** Called when the property name search box text changes. */
	void OnPropertySearchChanged(const FText& InText)
	{
		PropertySearchText = InText.ToString();
		ApplyFilters();
	}

	/** Called when the user clicks the Asset Path column header. Toggles between expanding and collapsing all tree nodes. */
	FReply OnToggleExpandAll()
	{
		bAllExpanded = !bAllExpanded;
		for (TSharedPtr<FMaterialDiffTreeNode> const& Node : AllRootNodes)
		{
			TreeView->SetItemExpansion(Node, bAllExpanded);
		}
		return FReply::Handled();
	}
};

} // namespace MaterialValidation

void SMaterialValidationGroupDiff::Construct(const FArguments& InArgs)
{
	OldGroup = InArgs._OldGroup;
	NewGroup = InArgs._NewGroup;

	TSharedPtr<MaterialValidation::SMaterialInstanceDiffTree> InstanceTree;

	TArray<TSharedPtr<MaterialValidation::FMaterialDiffStats>> DiffRows;
	if (OldGroup.IsValid() && NewGroup.IsValid())
	{
		DiffRows = MaterialValidation::BuildDiffRows(*OldGroup.Get(), *NewGroup.Get());
	}

	SAssignNew(InstanceTree, MaterialValidation::SMaterialInstanceDiffTree)
		.OldGroup(OldGroup.Get())
		.NewGroup(NewGroup.Get());

	TWeakPtr<MaterialValidation::SMaterialInstanceDiffTree> WeakInstanceTree = InstanceTree;

	ChildSlot
	[
		SNew(SSplitter)
		.Orientation(Orient_Vertical)
		+ SSplitter::Slot()
		.Value(0.6f)
		[
			SNew(MaterialValidation::SMaterialDiffTable)
				.InitialRows(DiffRows)
				.OnRowSelected_Lambda([WeakInstanceTree](TSharedPtr<MaterialValidation::FMaterialDiffStats> InRow)
				{
					if (TSharedPtr<MaterialValidation::SMaterialInstanceDiffTree> Tree = WeakInstanceTree.Pin())
					{
						if (InRow.IsValid())
						{
							Tree->SetMaterial(InRow->AssetPath);
						}
					}
				})
		]
		+ SSplitter::Slot()
		.Value(0.4f)
		[
			InstanceTree.ToSharedRef()
		]
	];
}

TSharedPtr<SWindow> SMaterialValidationGroupDiff::CreateDiffWindow(UMaterialValidationGroup const* InOldGroup, UMaterialValidationGroup const* InNewGroup, FRevisionInfo const& InOldRevision, FRevisionInfo const& InNewRevision)
{
	auto MakeRevisionLabel = [](FRevisionInfo const& Rev) -> FString
	{
		if (!Rev.Revision.IsEmpty())
		{
			return Rev.Revision;
		}
		if (Rev.Changelist > 0)
		{
			return FString::FromInt(Rev.Changelist);
		}
		return TEXT("Unknown");
	};

	const FText WindowTitle = FText::Format(
		LOCTEXT("DiffWindowTitle", "Material Validation Group Diff: {0} -> {1}"),
		FText::FromString(MakeRevisionLabel(InOldRevision)),
		FText::FromString(MakeRevisionLabel(InNewRevision)));

	TSharedPtr<SWindow> Window = SNew(SWindow)
		.Title(WindowTitle)
		.ClientSize(FVector2f(1400.f, 900.f));

	Window->SetContent(
		SNew(SMaterialValidationGroupDiff)
			.OldGroup(InOldGroup)
			.NewGroup(InNewGroup)
			.OldRevision(InOldRevision)
			.NewRevision(InNewRevision));

	const TSharedPtr<SWindow> ActiveModal = FSlateApplication::Get().GetActiveModalWindow();
	if (ActiveModal.IsValid())
	{
		FSlateApplication::Get().AddWindowAsNativeChild(Window.ToSharedRef(), ActiveModal.ToSharedRef());
	}
	else
	{
		FSlateApplication::Get().AddWindow(Window.ToSharedRef());
	}

	return Window;
}

#undef LOCTEXT_NAMESPACE
