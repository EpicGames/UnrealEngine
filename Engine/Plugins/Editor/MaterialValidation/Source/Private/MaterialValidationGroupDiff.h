// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinition.h"
#include "Widgets/SCompoundWidget.h"

class UMaterialValidationGroup;

/** Widget for the UMaterialValidationGroup asset diff window. */
class SMaterialValidationGroupDiff : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMaterialValidationGroupDiff) {}
		SLATE_ARGUMENT(UMaterialValidationGroup const*, OldGroup)
		SLATE_ARGUMENT(UMaterialValidationGroup const*, NewGroup)
		SLATE_ARGUMENT(FRevisionInfo, OldRevision)
		SLATE_ARGUMENT(FRevisionInfo, NewRevision)
	SLATE_END_ARGS()

	/** Build the widget layout from the provided group revisions. */
	void Construct(const FArguments& InArgs);

	/** Create and display a standalone diff window. */
	static TSharedPtr<SWindow> CreateDiffWindow(
		UMaterialValidationGroup const* InOldGroup,
		UMaterialValidationGroup const* InNewGroup,
		FRevisionInfo const& InOldRevision,
		FRevisionInfo const& InNewRevision);

private:
	TWeakObjectPtr<const UMaterialValidationGroup> OldGroup;
	TWeakObjectPtr<const UMaterialValidationGroup> NewGroup;
};
