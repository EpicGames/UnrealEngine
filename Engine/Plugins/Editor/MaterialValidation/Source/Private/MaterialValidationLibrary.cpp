// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialValidationLibrary.h"

#include "Algo/Sort.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Interfaces/IPluginManager.h"
#include "MaterialEditingLibrary.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialLayersFunctions.h"
#include "MaterialValidationConfig.h"
#include "MaterialValidationGroup.h"
#include "MaterialValidationLibraryTypes.h"
#include "MaterialValidationModule.h"
#include "Misc/ScopedSlowTask.h"
#include "MaterialCachedData.h"
#include "MaterialShared.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialValidationLibrary)

#define LOCTEXT_NAMESPACE "MaterialValidation"

namespace MaterialValidationLibrary {

/** Base material data necessary to calculate permutation id. */
struct FMaterialPermutationData
{
	/** Infos required to request per material instance static switch values. */
	TArray<FMaterialParameterInfo> StaticSwitchParameterInfo;
	/** Infos required to request per material instance component mask values. */
	TArray<FMaterialParameterInfo> ComponentMaskParameterInfo;
	/** Hash of all static switch and component mask GUIDs. */
	uint32 StaticPropertyLayoutHash;
};

/** Build the FMaterialPermutationData for a UMaterial. */
static void BuildPermutationData(UMaterial const& InBaseMaterial, FMaterialPermutationData& OutPermutationData)
{
	TArray<FGuid> StaticSwitchGuids;
	InBaseMaterial.GetAllStaticSwitchParameterInfo(OutPermutationData.StaticSwitchParameterInfo, StaticSwitchGuids);
	
	TArray<FGuid> ComponentMaskGuids;
	InBaseMaterial.GetAllStaticComponentMaskParameterInfo(OutPermutationData.ComponentMaskParameterInfo, ComponentMaskGuids);

	OutPermutationData.StaticPropertyLayoutHash = HashCombine(GetTypeHash(StaticSwitchGuids), GetTypeHash(ComponentMaskGuids));
}

/** Get the fully resolved static permutation values from a material or material instance. */
static FStaticPermutationProperties GetStaticPermutationProperties(UMaterialInterface const& InMaterial)
{
	FStaticPermutationProperties StaticProperties;
	StaticProperties.OpacityMaskClipValue = InMaterial.GetOpacityMaskClipValue();
	StaticProperties.BlendMode = InMaterial.GetBlendMode();
	StaticProperties.ShadingModel = InMaterial.IsShadingModelFromMaterialExpression() ? MSM_FromMaterialExpression : InMaterial.GetShadingModels().GetFirstShadingModel();
	StaticProperties.DitheredLODTransition = InMaterial.IsDitheredLODTransition();
	StaticProperties.bCastDynamicShadowAsMasked = InMaterial.GetCastDynamicShadowAsMasked();
	StaticProperties.TwoSided = InMaterial.IsTwoSided();
	StaticProperties.bIsThinSurface = InMaterial.IsThinSurface();
	StaticProperties.bOutputTranslucentVelocity = InMaterial.IsTranslucencyWritingVelocity();
	StaticProperties.bHasPixelAnimation = InMaterial.HasPixelAnimation();
	StaticProperties.bEnableTessellation = InMaterial.IsTessellationEnabled();
	StaticProperties.UsageFlags = InMaterial.GetUsageFlags();
	return StaticProperties;
}

/** Get the relevant static permutation values from a material instance. */
static FStaticPermutationProperties GetStaticPermutationPropertyOverrides(UMaterialInstance const& InMaterialInstance)
{
	FStaticPermutationProperties StaticProperties;
	StaticProperties.OpacityMaskClipValue = InMaterialInstance.BasePropertyOverrides.OpacityMaskClipValue;
	StaticProperties.BlendMode = InMaterialInstance.BasePropertyOverrides.BlendMode;
	StaticProperties.ShadingModel = InMaterialInstance.BasePropertyOverrides.ShadingModel;
	StaticProperties.DitheredLODTransition = InMaterialInstance.BasePropertyOverrides.DitheredLODTransition;
	StaticProperties.bCastDynamicShadowAsMasked = InMaterialInstance.BasePropertyOverrides.bCastDynamicShadowAsMasked;
	StaticProperties.TwoSided = InMaterialInstance.BasePropertyOverrides.TwoSided;
	StaticProperties.bIsThinSurface = InMaterialInstance.BasePropertyOverrides.bIsThinSurface;
	StaticProperties.bOutputTranslucentVelocity = InMaterialInstance.BasePropertyOverrides.bOutputTranslucentVelocity;
	StaticProperties.bHasPixelAnimation = InMaterialInstance.BasePropertyOverrides.bHasPixelAnimation;
	StaticProperties.bEnableTessellation = InMaterialInstance.BasePropertyOverrides.bEnableTessellation;
	StaticProperties.UsageFlags = InMaterialInstance.BasePropertyOverrides.UsageFlags;
	return StaticProperties;
}

/** Get the relevant static permutation override flags for values are enabled from a material instance. */
static FStaticPermutationPropertyOverrideFlags GetStaticPermutationPropertyOverrideFlags(UMaterialInstance const& InMaterialInstance)
{
	FStaticPermutationPropertyOverrideFlags OverrideFlags;
	OverrideFlags.bOverride_OpacityMaskClipValue = InMaterialInstance.BasePropertyOverrides.bOverride_OpacityMaskClipValue;
	OverrideFlags.bOverride_BlendMode = InMaterialInstance.BasePropertyOverrides.bOverride_BlendMode;
	OverrideFlags.bOverride_ShadingModel = InMaterialInstance.BasePropertyOverrides.bOverride_ShadingModel;
	OverrideFlags.bOverride_DitheredLODTransition = InMaterialInstance.BasePropertyOverrides.bOverride_DitheredLODTransition;
	OverrideFlags.bOverride_CastDynamicShadowAsMasked = InMaterialInstance.BasePropertyOverrides.bOverride_CastDynamicShadowAsMasked;
	OverrideFlags.bOverride_TwoSided = InMaterialInstance.BasePropertyOverrides.bOverride_TwoSided;
	OverrideFlags.bOverride_bIsThinSurface = InMaterialInstance.BasePropertyOverrides.bOverride_bIsThinSurface;
	OverrideFlags.bOverride_OutputTranslucentVelocity = InMaterialInstance.BasePropertyOverrides.bOverride_OutputTranslucentVelocity;
	OverrideFlags.bOverride_bHasPixelAnimation = InMaterialInstance.BasePropertyOverrides.bOverride_bHasPixelAnimation;
	OverrideFlags.bOverride_bEnableTessellation = InMaterialInstance.BasePropertyOverrides.bOverride_bEnableTessellation;
	OverrideFlags.Override_UsageFlags = InMaterialInstance.BasePropertyOverrides.bOverride_UsageFlags;
	return OverrideFlags;
}

/** Fill a FMaterialInstanceBasePropertyOverrides from FStaticPermutationProperties, setting all the override flags. */
FMaterialInstanceBasePropertyOverrides GetStaticPropertiesAsBasePropertyOverrides(FStaticPermutationProperties const& InStaticProperties)
{
	FMaterialInstanceBasePropertyOverrides BaseProperties;
	BaseProperties.bOverride_OpacityMaskClipValue = true;
	BaseProperties.OpacityMaskClipValue = InStaticProperties.OpacityMaskClipValue;
	BaseProperties.bOverride_BlendMode = true;
	BaseProperties.BlendMode = InStaticProperties.BlendMode;
	BaseProperties.bOverride_ShadingModel = true;
	BaseProperties.ShadingModel = InStaticProperties.ShadingModel;
	BaseProperties.bOverride_DitheredLODTransition = true;
	BaseProperties.DitheredLODTransition = InStaticProperties.DitheredLODTransition;
	BaseProperties.bOverride_CastDynamicShadowAsMasked = true;
	BaseProperties.bCastDynamicShadowAsMasked = InStaticProperties.bCastDynamicShadowAsMasked;
	BaseProperties.bOverride_TwoSided = true;
	BaseProperties.TwoSided = InStaticProperties.TwoSided;
	BaseProperties.bOverride_bIsThinSurface = true;
	BaseProperties.bIsThinSurface = InStaticProperties.bIsThinSurface;
	BaseProperties.bOverride_OutputTranslucentVelocity = true;
	BaseProperties.bOutputTranslucentVelocity = InStaticProperties.bOutputTranslucentVelocity;
	BaseProperties.bOverride_bHasPixelAnimation = true;
	BaseProperties.bHasPixelAnimation = InStaticProperties.bHasPixelAnimation;
	BaseProperties.bOverride_bEnableTessellation = true;
	BaseProperties.bEnableTessellation = InStaticProperties.bEnableTessellation;
	BaseProperties.bOverride_UsageFlags = ~0u;
	BaseProperties.UsageFlags = InStaticProperties.UsageFlags;
	return BaseProperties;
}

/** Return a hash built from any material layer settings on a material. */
static uint32 BuildLayerHash(UMaterialInterface const& InMaterial)
{
	FMaterialLayersFunctions MaterialLayers;
	if (InMaterial.GetMaterialLayers(MaterialLayers))
	{
		return GetTypeHash(MaterialLayers.GetStaticPermutationString());
	}
	return 0;
}

/** Extract AssetData from a UMaterial object. */
static void GetAssetData(
	UMaterial const& InMaterial, 
	FMaterialPermutationData const& InPermutationData, 
	FMaterialValidationAssetData_Material& OutAssetData)
{
	OutAssetData = {};

	// Gather static switches.
	{
		const uint32 StaticSwitchNum = InPermutationData.StaticSwitchParameterInfo.Num();
		const uint32 StaticSwitchBitArrayDwordNum = (StaticSwitchNum + 31) / 32;
		OutAssetData.StaticSwitchValues.SetNumZeroed(StaticSwitchBitArrayDwordNum);
		OutAssetData.StaticSwitchNum = StaticSwitchNum;

		for (uint32 Index = 0; Index < StaticSwitchNum; ++Index)
		{
			FMaterialParameterMetadata Meta;
			const bool bValid = InMaterial.GetParameterValue(EMaterialParameterType::StaticSwitch, InPermutationData.StaticSwitchParameterInfo[Index], Meta);
			const bool bValue = bValid && Meta.Value.AsStaticSwitch();
			if (bValue)
			{
				const uint32 BitArrayIndex = Index / 32;
				const uint32 BitIndex = Index % 32;
				OutAssetData.StaticSwitchValues[BitArrayIndex] |= 1 << BitIndex;
			}
		}
	}

	// Gather static component masks.
	{
		const uint32 ComponentMaskNum = InPermutationData.ComponentMaskParameterInfo.Num();
		const uint32 ComponentMaskBitArrayDwordNum = ((ComponentMaskNum * 4) + 31) / 32;
		OutAssetData.ComponentMaskValues.SetNumZeroed(ComponentMaskBitArrayDwordNum);
		OutAssetData.ComponentMaskNum = ComponentMaskNum;

		for (uint32 Index = 0; Index < ComponentMaskNum; ++Index)
		{
			FMaterialParameterMetadata Meta;
			const bool bValid = InMaterial.GetParameterValue(EMaterialParameterType::StaticComponentMask, InPermutationData.ComponentMaskParameterInfo[Index], Meta);
			const FStaticComponentMaskValue Value = bValid ? Meta.Value.AsStaticComponentMask() : FStaticComponentMaskValue();
			const uint32 ValueMask = (Value.R ? 1 : 0) | (Value.G ? 2 : 0) | (Value.B ? 4 : 0) | (Value.A ? 8 : 0);

			if (ValueMask)
			{
				const uint32 BitArrayIndex = Index / 8;
				const uint32 BitIndex = (Index * 4) % 32;
				OutAssetData.ComponentMaskValues[BitArrayIndex] |= ValueMask << BitIndex;
			}
		}
	}

	OutAssetData.StaticProperties = GetStaticPermutationProperties(InMaterial);

	OutAssetData.MaterialLayerHash = BuildLayerHash(InMaterial);
	
	OutAssetData.StaticPropertyLayoutHash = InPermutationData.StaticPropertyLayoutHash;
	OutAssetData.PermutationHash = MaterialValidation::BuildPermutationHash(OutAssetData.StaticProperties, OutAssetData.StaticSwitchValues, OutAssetData.ComponentMaskValues, OutAssetData.MaterialLayerHash);
}

/** Extract AssetData from a UMaterialInstance object. */
static void GetAssetData(
	UMaterialInstance const& InMaterialInstance, 
	FMaterialPermutationData const& InPermutationData,
	FMaterialValidationAssetData_MaterialInstance& OutAssetData)
{
	OutAssetData = {};

	UMaterialInterface const* ParentMaterial = InMaterialInstance.Parent;

	// We collect the static switch and component mask values to create the
	// permutation hash but only store the overridden values to the asset data.
	TArray<uint32> StaticSwitchValues;
	TArray<uint32> ComponentMaskValues;

	// Gather static switches.
	{
		const uint32 StaticSwitchNum = InPermutationData.StaticSwitchParameterInfo.Num();
		const uint32 StaticSwitchBitArrayDwordNum = (StaticSwitchNum + 31) / 32;
		OutAssetData.StaticSwitchOverrideMask.SetNumZeroed(StaticSwitchBitArrayDwordNum);
		OutAssetData.StaticSwitchOverrideValues.SetNumZeroed(StaticSwitchBitArrayDwordNum);
		StaticSwitchValues.SetNumZeroed(StaticSwitchBitArrayDwordNum);

		for (uint32 Index = 0; Index < StaticSwitchNum; ++Index)
		{
			FMaterialParameterMetadata InstanceMeta;
			const bool bInstanceValid = InMaterialInstance.GetParameterValue(EMaterialParameterType::StaticSwitch, InPermutationData.StaticSwitchParameterInfo[Index], InstanceMeta);
			const bool Value = bInstanceValid && InstanceMeta.Value.AsStaticSwitch();

			FMaterialParameterMetadata ParentMeta;
			const bool bParentValid = ParentMaterial->GetParameterValue(EMaterialParameterType::StaticSwitch, InPermutationData.StaticSwitchParameterInfo[Index], ParentMeta);
			const bool ParentValue = bParentValid && ParentMeta.Value.AsStaticSwitch();

			const uint32 BitArrayIndex = Index / 32;
			const uint32 BitIndex = Index % 32;

			if (Value)
			{
				StaticSwitchValues[BitArrayIndex] |= Value << BitIndex;
			}
			if (Value != ParentValue)
			{
				OutAssetData.StaticSwitchOverrideMask[BitArrayIndex] |= 1 << BitIndex;
				OutAssetData.StaticSwitchOverrideValues[BitArrayIndex] |= Value << BitIndex;
			}
		}
	}

	// Gather static component masks.
	{
		const uint32 ComponentMaskNum = InPermutationData.ComponentMaskParameterInfo.Num();
		const uint32 ComponentMaskBitValueArrayDwordNum = ((ComponentMaskNum * 4) + 31) / 32;
		OutAssetData.ComponentMaskOverrideValues.SetNumZeroed(ComponentMaskBitValueArrayDwordNum);
		OutAssetData.ComponentMaskOverrideMask.SetNumZeroed(ComponentMaskBitValueArrayDwordNum);
		ComponentMaskValues.SetNumZeroed(ComponentMaskBitValueArrayDwordNum);

		for (uint32 Index = 0; Index < ComponentMaskNum; ++Index)
		{
			FMaterialParameterMetadata InstanceMeta;
			const bool bInstanceValid = InMaterialInstance.GetParameterValue(EMaterialParameterType::StaticComponentMask, InPermutationData.ComponentMaskParameterInfo[Index], InstanceMeta);
			const FStaticComponentMaskValue Value = bInstanceValid ? InstanceMeta.Value.AsStaticComponentMask() : FStaticComponentMaskValue();
			const uint32 ValueMask = (Value.R ? 1 : 0) | (Value.G ? 2 : 0) | (Value.B ? 4 : 0) | (Value.A ? 8 : 0);

			FMaterialParameterMetadata ParentMeta;
			const bool bParentValid = ParentMaterial->GetParameterValue(EMaterialParameterType::StaticComponentMask, InPermutationData.ComponentMaskParameterInfo[Index], ParentMeta);
			const FStaticComponentMaskValue ParentValue = bParentValid ? ParentMeta.Value.AsStaticComponentMask() : FStaticComponentMaskValue();

			const uint32 BitArrayIndex = Index / 8;
			const uint32 BitIndex = (Index * 4) % 32;

			if (ValueMask)
			{
				ComponentMaskValues[BitArrayIndex] |= ValueMask << BitIndex;
			}
			if (Value != ParentValue)
			{
				OutAssetData.ComponentMaskOverrideMask[BitArrayIndex] |= 0xf << BitIndex;
				OutAssetData.ComponentMaskOverrideValues[BitArrayIndex] |= ValueMask << BitIndex;
			}
		}
	}

	FStaticPermutationProperties StaticProperties = GetStaticPermutationProperties(InMaterialInstance);
	OutAssetData.StaticProperties = GetStaticPermutationPropertyOverrides(InMaterialInstance);
	OutAssetData.StaticPropertyOverrideFlags = GetStaticPermutationPropertyOverrideFlags(InMaterialInstance);

	OutAssetData.MaterialLayerHash = BuildLayerHash(InMaterialInstance);
	
	OutAssetData.PermutationHash = MaterialValidation::BuildPermutationHash(StaticProperties, StaticSwitchValues, ComponentMaskValues, OutAssetData.MaterialLayerHash);
}

/** Build hash for a material instance which will match for all instances that generate the same shader permutations. */
static uint32 BuildPermutationHash(UMaterialInstanceConstant const& InMaterialInstance, UMaterial const& InBaseMaterial)
{
	FMaterialPermutationData PermutationData;
	BuildPermutationData(InBaseMaterial, PermutationData);

	FMaterialValidationAssetData_MaterialInstance InstanceAssetData;
	GetAssetData(InMaterialInstance, PermutationData, InstanceAssetData);

	return InstanceAssetData.PermutationHash;
}

/** Apply overrides for bit mask arrays such as those used by static switch and component masks. */
static void ApplyBitMaskOverrides(TConstArrayView<uint32> InParentValues, TConstArrayView<uint32> InChildOverridesMasks, TConstArrayView<uint32> InChildOverrideValues, TArray<uint32>& OutValues, TArray<uint32>& OutModified)
{
	const int32 NumDwords = InParentValues.Num();
	check(NumDwords == InChildOverridesMasks.Num() && NumDwords == InChildOverrideValues.Num());
	OutValues.SetNumUninitialized(NumDwords);
	OutModified.SetNumUninitialized(NumDwords);

	for (int32 Index = 0; Index < NumDwords; ++Index)
	{
		OutValues[Index] = (InParentValues[Index] & ~InChildOverridesMasks[Index]) | (InChildOverridesMasks[Index] & InChildOverrideValues[Index]);
		OutModified[Index] = OutValues[Index] ^ InParentValues[Index];
	}
}

/** Apply material base property overrides to generate the child base property values from the parent values and the child overrides. */
static void ApplyBasePropertyOverrides(
	FStaticPermutationProperties const& InParentProperties, 
	FStaticPermutationPropertyOverrideFlags const& InChildOverrideFlags,
	FStaticPermutationProperties const& InChildOverrideProperties,
	FStaticPermutationProperties& OutProperties,
	FStaticPermutationPropertyOverrideFlags& OutPropertiesModified)
{
	OutProperties.TwoSided = InChildOverrideFlags.bOverride_TwoSided ? InChildOverrideProperties.TwoSided : InParentProperties.TwoSided;
	OutProperties.bIsThinSurface = InChildOverrideFlags.bOverride_bIsThinSurface ? InChildOverrideProperties.bIsThinSurface : InParentProperties.bIsThinSurface;
	OutProperties.DitheredLODTransition = InChildOverrideFlags.bOverride_DitheredLODTransition ? InChildOverrideProperties.DitheredLODTransition : InParentProperties.DitheredLODTransition;
	OutProperties.bCastDynamicShadowAsMasked = InChildOverrideFlags.bOverride_CastDynamicShadowAsMasked ? InChildOverrideProperties.bCastDynamicShadowAsMasked : InParentProperties.bCastDynamicShadowAsMasked;
	OutProperties.bOutputTranslucentVelocity = InChildOverrideFlags.bOverride_OutputTranslucentVelocity ? InChildOverrideProperties.bOutputTranslucentVelocity : InParentProperties.bOutputTranslucentVelocity;
	OutProperties.bHasPixelAnimation = InChildOverrideFlags.bOverride_bHasPixelAnimation ? InChildOverrideProperties.bHasPixelAnimation : InParentProperties.bHasPixelAnimation;
	OutProperties.bEnableTessellation = InChildOverrideFlags.bOverride_bEnableTessellation ? InChildOverrideProperties.bEnableTessellation : InParentProperties.bEnableTessellation;
	OutProperties.BlendMode = InChildOverrideFlags.bOverride_BlendMode ? InChildOverrideProperties.BlendMode : InParentProperties.BlendMode;
	// Note that we don't allow override with MSM_FromMaterialExpression.
	OutProperties.ShadingModel = InChildOverrideFlags.bOverride_ShadingModel && InChildOverrideProperties.ShadingModel != MSM_FromMaterialExpression ? InChildOverrideProperties.ShadingModel : InParentProperties.ShadingModel;
	OutProperties.OpacityMaskClipValue = InChildOverrideFlags.bOverride_OpacityMaskClipValue ? InChildOverrideProperties.OpacityMaskClipValue : InParentProperties.OpacityMaskClipValue;
	OutProperties.UsageFlags = (InParentProperties.UsageFlags & ~InChildOverrideFlags.Override_UsageFlags) | (InChildOverrideFlags.Override_UsageFlags & InChildOverrideProperties.UsageFlags);

	// Return what child properties are actually different
	OutPropertiesModified.bOverride_OpacityMaskClipValue = OutProperties.OpacityMaskClipValue != InParentProperties.OpacityMaskClipValue;
	OutPropertiesModified.bOverride_BlendMode = OutProperties.BlendMode != InParentProperties.BlendMode;
	OutPropertiesModified.bOverride_ShadingModel = OutProperties.ShadingModel != InParentProperties.ShadingModel;
	OutPropertiesModified.bOverride_DitheredLODTransition = OutProperties.DitheredLODTransition != InParentProperties.DitheredLODTransition;
	OutPropertiesModified.bOverride_CastDynamicShadowAsMasked = OutProperties.bCastDynamicShadowAsMasked != InParentProperties.bCastDynamicShadowAsMasked;
	OutPropertiesModified.bOverride_TwoSided = OutProperties.TwoSided != InParentProperties.TwoSided;
	OutPropertiesModified.bOverride_bIsThinSurface = OutProperties.bIsThinSurface != InParentProperties.bIsThinSurface;
	OutPropertiesModified.bOverride_OutputTranslucentVelocity = OutProperties.bOutputTranslucentVelocity != InParentProperties.bOutputTranslucentVelocity;
	OutPropertiesModified.bOverride_bHasPixelAnimation = OutProperties.bHasPixelAnimation != InParentProperties.bHasPixelAnimation;
	OutPropertiesModified.bOverride_bEnableTessellation = OutProperties.bEnableTessellation != InParentProperties.bEnableTessellation;
	OutPropertiesModified.Override_UsageFlags = OutProperties.UsageFlags ^ InParentProperties.UsageFlags;
}

/** Gather all search and exclude directory paths for a group. */
static void GetSearchPaths(UMaterialValidationGroup const& InGroup, TArray<FString>& OutPaths, TArray<FString>& OutExcludePaths)
{
	for (FDirectoryPath Path : InGroup.MaterialPaths)
	{
		OutPaths.Add(Path.Path);
	}

	for (FDirectoryPath Path : InGroup.MaterialExcludePaths)
	{
		OutExcludePaths.Add(Path.Path);
	}

	if (InGroup.bAddProjectPath)
	{
		OutPaths.Add(TEXT("/Game/"));

		//todo: Expose UEditorValidatorSubsystem::ExcludedDirectories and skip them all.
		OutExcludePaths.Add(TEXT("/Game/Developers/"));
	}

	if (InGroup.bAddProjectPluginPaths)
	{
		IPluginManager& PluginManager = IPluginManager::Get();
		TArray<TSharedRef<IPlugin>> Plugins = PluginManager.GetDiscoveredPlugins();
		for (TSharedRef<IPlugin> const& Plugin : Plugins)
		{
			if (Plugin->GetLoadedFrom() == EPluginLoadedFrom::Project && Plugin->CanContainContent())
			{
				TStringBuilder<128> Builder;
				Builder.AppendChar(TEXT('/')).Append(Plugin->GetName()).AppendChar(TEXT('/'));
				OutPaths.Add(*Builder);
			}
		}
	}
}

/** Returns true if the PackagePath is found under any of the paths in the InPaths array. */
static bool IsPackageInPaths(FString const& PackagePath, TConstArrayView<FString> InPaths)
{
	for (FString const& Path : InPaths)
	{
		if (PackagePath.StartsWith(Path))
		{
			return true;
		}
	}
	return false;
}

/** Gather all assets of a given class using the given search and exclued paths. */
static void GetAssetsFromSearchPaths(UClass* InClass, TConstArrayView<FString> InSearchPaths, TConstArrayView<FString> InExcludePaths, TArray<FAssetData>& OutAssetDatas)
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

	for (FString const& SearchPath : InSearchPaths)
	{
		FARFilter Filter;
		Filter.PackagePaths.Add(*SearchPath);
		Filter.ClassPaths.Add(InClass->GetClassPathName());
		Filter.bRecursivePaths = true;

		const int32 BaseAddedIndex = OutAssetDatas.Num();
		AssetRegistryModule.Get().GetAssets(Filter, OutAssetDatas);

		// FARFilter doesn't support exclude paths. So we have to manually prune :(
		TArray<FString> RelevantExcludePaths;
		for (FString const& ExcludePath : InExcludePaths)
		{
			if (ExcludePath.StartsWith(SearchPath))
			{
				RelevantExcludePaths.Add(ExcludePath);
			}
		}
		if (RelevantExcludePaths.Num() > 0)
		{
			for (int32 Index = OutAssetDatas.Num() - 1; Index >= BaseAddedIndex; --Index)
			{
				FAssetData const& AssetData = OutAssetDatas[Index];
				FString PackagePath = AssetData.PackagePath.ToString();
				if (IsPackageInPaths(PackagePath, RelevantExcludePaths))
				{
					OutAssetDatas.RemoveAtSwap(Index, 1, EAllowShrinking::No);
				}
			}
		}
	}
}

/** Convert a string path to a FSoftObjectPath taking into account any asset redirectors. */
static FSoftObjectPath ConvertStringPathToSoftObjectPath(FString const& InAssetPath)
{
	FSoftObjectPath CurrentPath(InAssetPath);

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	const int32 MaxRedirects = 10;
	for (int32 RedirectDepth = 0; RedirectDepth < MaxRedirects; ++RedirectDepth)
	{
		FAssetData AssetData = AssetRegistry.GetAssetByObjectPath(CurrentPath);
		if (!AssetData.IsValid())
		{
			break;
		}

		if (!AssetData.IsRedirector())
		{
			break;
		}

		FString RedirectedPath;
		if (!AssetData.GetTagValue("DestinationObject", RedirectedPath))
		{
			break;
		}

		CurrentPath = FSoftObjectPath(RedirectedPath);
	}

	return CurrentPath;
}

/** Find the material description in a group that matches a FSoftObjectPath taking into account any asset redirectors. */
static FMaterialValidationDesc const* FindMaterialDescFromSoftObjectPath(UMaterialValidationGroup const& InGroup, FSoftObjectPath const& InSoftObjectPath)
{
	// First look at for simple converted value of the soft object path.
	FString ConvertedString = InSoftObjectPath.ToString();
	if (FMaterialValidationDesc const* MaterialDesc = InGroup.Materials.Find(ConvertedString))
	{
		return MaterialDesc;
	}

	// If not found then a redirector might be in play so do a slow find over all stored material paths.
	for (TPair<FString, FMaterialValidationDesc> const& It : InGroup.Materials)
	{
		if (MaterialValidationLibrary::ConvertStringPathToSoftObjectPath(It.Key) == InSoftObjectPath)
		{
			return & It.Value;
		}
	}

	return nullptr;
}

static bool FindMaterialInstanceIndexFromSoftObjectPath(FMaterialValidationDesc const& InMaterialDesc, FSoftObjectPath const& InSoftObjectPath, int32& OutIndex)
{
	// First look at for simple converted value of the soft object path.
	FString ConvertedString = InSoftObjectPath.ToString();
	if (int32 const* IndexPtr = InMaterialDesc.MaterialInstances.Find(ConvertedString))
	{
		OutIndex = *IndexPtr;
		return true;
	}

	// If not found then a redirector might be in play so do a slow find over all stored material instance paths.
	for (TPair<FString, int32> const& It : InMaterialDesc.MaterialInstances)
	{
		if (MaterialValidationLibrary::ConvertStringPathToSoftObjectPath(It.Key) == InSoftObjectPath)
		{
			OutIndex = It.Value;
			return true;
		}
	}

	return false;
}

/** 
 * Add the package path extension to a FName package path that has a missing extension. 
 * todo: Ideally we would avoid this by only passing around package paths in one canonical form.
 */
FString AddPackagePathExtension(FName InPackagePath)
{
	FString PackagePath = InPackagePath.ToString();

	int32 LastSlashIndex;
	if (!PackagePath.FindLastChar('/', LastSlashIndex))
	{
		LastSlashIndex = -1;
	}

	int32 LastPeriodIndex;
	if (PackagePath.FindLastChar('.', LastPeriodIndex) && LastPeriodIndex > LastSlashIndex)
	{
		return PackagePath;
	}

	FString AssetName = PackagePath.Mid(LastSlashIndex + 1);
	if (AssetName.IsEmpty())
	{
		return PackagePath;
	}
	
	return FString::Printf(TEXT("%s.%s"), *PackagePath, *AssetName);
}

/** 
 * Traverse the instance hierarchy for a material and write to a depth first sorted array of package names.
 * The array will have the input material package name as its first entry.
 */
static void GetMaterialInstanceHierarchy(UMaterial const& InMaterial, TConstArrayView<FString> InExcludePaths, TArray<FName>& OutPackageNames, TArray<int32>& OutHierarchyDepths)
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	const FAssetData MaterialAssetData = AssetRegistry.GetAssetByObjectPath(InMaterial.GetPathName());
	if (!MaterialAssetData.IsValid())
	{
		return;
	}

	// Maintain a stack of names to output.
	TArray<FName> NameStack;
	NameStack.Push(MaterialAssetData.PackageName);
	// Maintain a stack of depths so that we can output those to OutHierarchyDepths.
	TArray<int32> DepthStack;
	DepthStack.Push(0);
	// Maintain a stack of redirector flags. We follow the redirectors but don't add them to the output.
	TArray<bool> RedirectorStack;
	RedirectorStack.Push(false);
	// Maintain an array of names we have seen but discarded to avoid cyclic iteration.
	TArray<FName> DiscardedPackageNames;

	while (!NameStack.IsEmpty())
	{
		// Pop from stack and add to output.
		const FName ParentPackageName = NameStack.Pop(EAllowShrinking::No);
		const int32 ParentPackageDepth = DepthStack.Pop(EAllowShrinking::No);
		const bool bParentIsRedirector = RedirectorStack.Pop(EAllowShrinking::No);

		if (bParentIsRedirector)
		{
			DiscardedPackageNames.Add(ParentPackageName);
		}
		else
		{
			OutPackageNames.Add(ParentPackageName);
			OutHierarchyDepths.Add(ParentPackageDepth);
		}

		// Push children onto our stack.
		TArray<FName> ChildPackageNames;
		AssetRegistry.GetReferencers(ParentPackageName, ChildPackageNames);

		for (const FName& PackageName : ChildPackageNames)
		{
			if (OutPackageNames.Contains(PackageName) || NameStack.Contains(PackageName) || DiscardedPackageNames.Contains(PackageName) ||  IsPackageInPaths(PackageName.ToString(), InExcludePaths))
			{
				continue;
			}

			TArray<FAssetData> AssetDatas;
			AssetRegistry.GetAssetsByPackageName(PackageName, AssetDatas, true);

			for (FAssetData const& AssetData : AssetDatas)
			{
				if (!AssetData.IsValid())
				{
					continue;
				}

				if (AssetData.IsRedirector())
				{
					NameStack.Push(PackageName);
					DepthStack.Push(ParentPackageDepth);
					RedirectorStack.Push(true);
					break;
				}

				if (AssetData.IsInstanceOf(UMaterialInstanceConstant::StaticClass()))
				{
					NameStack.Push(PackageName);
					DepthStack.Push(ParentPackageDepth + 1);
					RedirectorStack.Push(false);
					break;
				}
			}
		}
	}

	UE_LOGF(LogMaterialValidation, Display, "Found %d material instances for %ls.", OutPackageNames.Num() - 1, *InMaterial.GetName());
}

/**
 * Load the instances for a material from an array of package names.
 * Returns a material instance object array with the same indexing (with nullptr if we can't load).
 * Loading the assets can be very slow for a material with many references that may also load/compile textures etc.
 */
static void LoadMaterialInstances(UMaterial const& InMaterial, TConstArrayView<FName> InPackageNames, TArray<UMaterialInstanceConstant*>& OutMaterialInstances)
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	FScopedSlowTask SlowTask(static_cast<float>(InPackageNames.Num()), LOCTEXT("LoadingMaterialInstances", "Loading Material Instances..."));

	OutMaterialInstances.SetNumZeroed(InPackageNames.Num());

	for (int32 PackageIndex = 0; PackageIndex < InPackageNames.Num(); ++PackageIndex)
	{
		SlowTask.EnterProgressFrame();

		// Each package name may map to several FAssetData so iterate them all to find one that has our base material.
		TArray<FAssetData> AssetDatas;
		AssetRegistry.GetAssetsByPackageName(InPackageNames[PackageIndex], AssetDatas, true);
		for (FAssetData const& AssetData : AssetDatas)
		{
			if (!AssetData.IsValid())
			{
				continue;
			}
			if (!AssetData.IsInstanceOf(UMaterialInstanceConstant::StaticClass()))
			{
				continue;
			}
			if (UMaterialInstanceConstant* MaterialInstance = Cast<UMaterialInstanceConstant>(AssetData.GetAsset()))
			{
				if (MaterialInstance->GetMaterial() == &InMaterial)
				{
					OutMaterialInstances[PackageIndex] = MaterialInstance;
					break;
				}
			}
		}
	}
}

/**
 * Get the (non-deduplicated) number of shaders that a single material or material instance will generate.
 * Note: This is only calculated for the current (editor) platform and probably contains debug shaders that aren't cooked.
 */
static int32 GetNumShaderTypes(UMaterialInterface const& InMaterial)
{
	// todo: GetNumShaderTypes() isn't const, but maybe could be?
	return UMaterialEditingLibrary::GetNumShaderTypes(const_cast<UMaterialInterface*>(&InMaterial));
}

/** 
 * Update the FMaterialValidationDesc for a material.
 * This contains asset data for all material instances and information about their unique permutations.
 * Internally this calls LoadMaterialInstances() that is a potentially slow function to load all the material instance assets.
 */
static void UpdateValidMaterialInstances(UMaterial const& InMaterial, TConstArrayView<FString> InExcludePaths, FMaterialValidationDesc& OutMaterialDesc)
{
	OutMaterialDesc = {};

	// Initialize shader counts.
	OutMaterialDesc.NumShadersBase = GetNumShaderTypes(InMaterial);
	OutMaterialDesc.NumShadersTotal = OutMaterialDesc.NumShadersBase;

	// Get all material instance packages.
	// PackageNames will contain InMaterial in the first entry.
	TArray<FName> PackageNames;
	TArray<int32> HierarchyDepths;
	GetMaterialInstanceHierarchy(InMaterial, InExcludePaths, PackageNames, HierarchyDepths);

	if (PackageNames.Num() <= 1)
	{
		return;
	}

	// Load the material instance assets.
	TArray<UMaterialInstanceConstant*> MaterialInstances;
	LoadMaterialInstances(InMaterial, PackageNames, MaterialInstances);

	// Get the material asset data.
	FMaterialPermutationData PermutationData;
	BuildPermutationData(InMaterial, PermutationData);
	GetAssetData(InMaterial, PermutationData, OutMaterialDesc.AssetData);

	// Walk the material instance hierarchy.
	// We keep a stack of permutation hashes to detect when a child has the same hash as the parent.
	TArray<uint32> PermutationHashStack;
	PermutationHashStack.Push(OutMaterialDesc.AssetData.PermutationHash);

	for (int32 Index = 1; Index < MaterialInstances.Num(); ++Index)
	{
		const int32 CurrentDepth = HierarchyDepths[Index];
		PermutationHashStack.SetNumZeroed(CurrentDepth + 1);

		UMaterialInstance const* MaterialInstance = MaterialInstances[Index];
		if (MaterialInstance == nullptr)
		{
			continue;
		}

		// Add the material instance to the map.
		const FSoftObjectPath SoftPackagePath(MaterialInstance);
		const FString PackagePath = SoftPackagePath.ToString();

		int32& AssetDataIndex = OutMaterialDesc.MaterialInstances.FindOrAdd(PackagePath);
		AssetDataIndex = INDEX_NONE;

		// Get the material instance asset data.
		FMaterialValidationAssetData_MaterialInstance MaterialInstanceAssetData;
		GetAssetData(*MaterialInstance, PermutationData, MaterialInstanceAssetData);
		const uint32 Hash = MaterialInstanceAssetData.PermutationHash;

		// Only store the asset data if the child differs from the parent.
		// Otherwise we can assume the asset data is default and save some asset size by not storing it.
		if (Hash != PermutationHashStack[CurrentDepth - 1])
		{
			AssetDataIndex = OutMaterialDesc.MaterialInstanceAssetDatas.Add(MaterialInstanceAssetData);
		}

		// If this is a new permutation hash then we store it and add to the shader count.
		if (Hash != OutMaterialDesc.AssetData.PermutationHash && !OutMaterialDesc.PermutationHashes.Contains(Hash))
		{
			UE_LOGF(LogMaterialValidation, Log, "Add material instance '%ls' to permutaton list.", *PackagePath);
			OutMaterialDesc.PermutationHashes.Add(Hash);
			OutMaterialDesc.NumShadersTotal += GetNumShaderTypes(*MaterialInstance);
		}

		PermutationHashStack[CurrentDepth] = Hash;
	}
}

/** 
 * Get the asset data associated with a package path.
 * We get this from either the material validation group or from the object directly.
 * If we return false then it is likely that the material instance exists but:
 * - It wasn't stored in the validation group (it was probably created after last validation group update), and
 * - It isn't in the curent validation changelist.
 */
static bool GetMaterialInstanceAssetData(
	FString const& InPackagePath,
	UMaterialInstanceConstant const* InMaterialInstance,
	FMaterialValidationDesc const* InMaterialDesc,
	FMaterialPermutationData const& InMaterialPermutationData,
	FMaterialValidationAssetData_MaterialInstance& OutAssetData)
{
	// If we were provided with a modified object then use that.
	if (InMaterialInstance != nullptr)
	{
		GetAssetData(*InMaterialInstance, InMaterialPermutationData, OutAssetData);
		return true;
	}

	// Otherwise use asset data from the material validation group.
	if (InMaterialDesc != nullptr)
	{
		int32 AssetDataIndex = INDEX_NONE;
		if (MaterialValidationLibrary::FindMaterialInstanceIndexFromSoftObjectPath(*InMaterialDesc, InPackagePath, AssetDataIndex))
		{
			if (AssetDataIndex == INDEX_NONE)
			{
				// Set up a default asset data with the right sizes.
				const int32 StaticSwitchArrayNum = InMaterialDesc->AssetData.StaticSwitchValues.Num();
				OutAssetData.StaticSwitchOverrideMask.SetNumZeroed(StaticSwitchArrayNum);
				OutAssetData.StaticSwitchOverrideValues.SetNumZeroed(StaticSwitchArrayNum);
				const int32 ComponentMaskArrayNum = InMaterialDesc->AssetData.ComponentMaskValues.Num();
				OutAssetData.ComponentMaskOverrideMask.SetNumZeroed(ComponentMaskArrayNum);
				OutAssetData.ComponentMaskOverrideValues.SetNumZeroed(ComponentMaskArrayNum);
				return true;
			}
			else if (InMaterialDesc->MaterialInstanceAssetDatas.IsValidIndex(AssetDataIndex))
			{
				OutAssetData = InMaterialDesc->MaterialInstanceAssetDatas[AssetDataIndex];
				return true;
			}
		}
	}

	return false;
}

static FStaticParameterSet BuildStaticParamSetFromValidationAsset(UMaterial& InBaseMaterial, FMaterialValidationAssetData_MaterialInstance const& InAssetData)
{
	FStaticParameterSet StaticParameterSet;

	// Fill the static switch parameters.
	TArray<FMaterialParameterInfo> StaticSwitchParameterInfo;
	TArray<FGuid> StaticSwitchGuids;
	InBaseMaterial.GetAllStaticSwitchParameterInfo(StaticSwitchParameterInfo, StaticSwitchGuids);

	for (uint32 Index = 0; Index < (uint32)StaticSwitchParameterInfo.Num(); ++Index)
	{
		const bool bValue = (InAssetData.StaticSwitchOverrideValues[Index / 32] & (1 << Index)) != 0;
		FMaterialParameterMetadata ParameterMetadata(bValue);
		ParameterMetadata.ExpressionGuid = StaticSwitchGuids[Index];
		StaticParameterSet.SetParameterValue(StaticSwitchParameterInfo[Index], ParameterMetadata, EMaterialSetParameterValueFlags::None);
	}

	// Fill the component mask parameters.
	TArray<FMaterialParameterInfo> ComponentMaskParameterInfo;
	TArray<FGuid> ComponentMaskGuids;
	InBaseMaterial.GetAllStaticComponentMaskParameterInfo(ComponentMaskParameterInfo, ComponentMaskGuids);

	for (uint32 Index = 0; Index < (uint32)ComponentMaskParameterInfo.Num(); ++Index)
	{
		const uint32 ValueMask = (InAssetData.ComponentMaskOverrideValues[Index / 8] >> ((Index % 8) * 4)) & 0xf;
		const FStaticComponentMaskValue Value(ValueMask & 1, ValueMask & 2, ValueMask & 4, ValueMask & 8);
		FMaterialParameterMetadata ParameterMetadata(Value);
		ParameterMetadata.ExpressionGuid = ComponentMaskGuids[Index];
		StaticParameterSet.SetParameterValue(ComponentMaskParameterInfo[Index], ParameterMetadata, EMaterialSetParameterValueFlags::None);
	}

	//todo: We should handle material layer overrides too.
	//When that happens, FMaterialCachedExpressionData::FunctionInfos should also be copied over for DDC queries to succeed.

	return StaticParameterSet;
}


/**
 * Create a transient material instance directly from an AssetData.
 * This can be used to evaluate shader counts for an asset without loading it, and paying the cost of loading references etc.
 * We do need the base material to be loaded, since we use it to create a temporary material instance.
 */
static UMaterialInstanceConstant* CreateMaterialInstance(UMaterial& InBaseMaterial, FMaterialValidationAssetData_MaterialInstance const& InAssetData)
{
	// Create a temporary UMaterialInstanceConstant parented to our base material.
	UMaterialInstanceConstant* MaterialInstance = NewObject<UMaterialInstanceConstant>();
	MaterialInstance->SetParentEditorOnly(&InBaseMaterial);

	// Apply all of the static properties.
	FMaterialInstanceBasePropertyOverrides BaseProperties = GetStaticPropertiesAsBasePropertyOverrides(InAssetData.StaticProperties);

	FStaticParameterSet StaticParamSet = MaterialValidationLibrary::BuildStaticParamSetFromValidationAsset(InBaseMaterial, InAssetData);
	MaterialInstance->SetPermutationParameters(StaticParamSet, BaseProperties);
	MaterialInstance->bHasStaticPermutationResource = true;

	return MaterialInstance;
}

} // namespace MaterialValidationLibrary

UMaterialValidationLibrary::UMaterialValidationLibrary(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UMaterialValidationLibrary::GetAllGroups(TArray<UMaterialValidationGroup*>& OutGroups, bool bInSyncLoad)
{
	UMaterialValidationConfig const* Config = GetDefault<UMaterialValidationConfig>();
	if (!Config->bEnable)
	{
		return;
	}

	for (TSoftObjectPtr<UMaterialValidationGroup> GroupSoftPtr : Config->Groups)
	{
		if (bInSyncLoad)
		{
			if (UMaterialValidationGroup* Group = GroupSoftPtr.LoadSynchronous())
			{
				OutGroups.Add(Group);
			}
		}
		else
		{
			if (GroupSoftPtr.IsValid())
			{
				if (UMaterialValidationGroup* Group = GroupSoftPtr.Get())
				{
					OutGroups.Add(Group);
				}
			}
			else if (!GroupSoftPtr.IsNull())
			{
				// Trigger async load in anticipation of being called again.
				GroupSoftPtr.LoadAsync({});
			}
		}
	}
}

void UMaterialValidationLibrary::ResetGroup(UMaterialValidationGroup* InGroup)
{
	if (InGroup != nullptr)
	{
		InGroup->Materials.Reset();
	}
}

void UMaterialValidationLibrary::AddMissingMaterialsToGroup(UMaterialValidationGroup* InGroup)
{
	if (InGroup == nullptr)
	{
		return;
	}

	// Iterate over search paths and gather all UMaterials found in the asset registry.
	TArray<FString> SearchPaths;
	TArray<FString> ExcludePaths;
	MaterialValidationLibrary::GetSearchPaths(*InGroup, SearchPaths, ExcludePaths);

	TArray<FAssetData> AssetDatas;
	MaterialValidationLibrary::GetAssetsFromSearchPaths(UMaterial::StaticClass(), SearchPaths, ExcludePaths, AssetDatas);
		
	UE_LOGF(LogMaterialValidation, Display, "Found %d materials in %d directories.", AssetDatas.Num(), SearchPaths.Num());

	// Iterate over UMaterial assets and add them if not found.
	int32 bNumAdded = 0;
	for (FAssetData const& AssetData : AssetDatas)
	{
		FString MaterialPath = AssetData.ToSoftObjectPath().ToString();

		if (!InGroup->Materials.Contains(MaterialPath))
		{
			UE_LOGF(LogMaterialValidation, Log, "Adding material '%ls' to '%ls'.", *AssetData.AssetName.ToString(), *InGroup->GetName());
			InGroup->Materials.Add(MaterialPath);
			bNumAdded++;
		}
	}

	if (bNumAdded)
	{
		InGroup->Modify();
	}
}

void UMaterialValidationLibrary::RemoveInvalidMaterialsFromGroup(UMaterialValidationGroup* InGroup)
{
	if (InGroup == nullptr)
	{
		return;
	}

	TArray<FString> SearchPaths;
	TArray<FString> ExcludePaths;
	MaterialValidationLibrary::GetSearchPaths(*InGroup, SearchPaths, ExcludePaths);

	TArray<FString> ToRemove;
	for (TPair<FString, FMaterialValidationDesc>& It : InGroup->Materials)
	{
		const FSoftObjectPath SoftPackagePath = MaterialValidationLibrary::ConvertStringPathToSoftObjectPath(It.Key);

		FPackagePath PackagePath;
		if (!FPackagePath::TryFromPackageName(SoftPackagePath.GetLongPackageName(), PackagePath))
		{
			ToRemove.Add(It.Key);
			continue;
		}

		if (!FPackageName::DoesPackageExist(PackagePath))
		{
			ToRemove.Add(It.Key);
			continue;
		}

		const FString PackageName = PackagePath.GetPackageName();
		if (MaterialValidationLibrary::IsPackageInPaths(PackageName, ExcludePaths))
		{
			ToRemove.Add(It.Key);
			continue;
		}

		if (!MaterialValidationLibrary::IsPackageInPaths(PackageName, SearchPaths))
		{
			ToRemove.Add(It.Key);
		}
	}

	UE_LOGF(LogMaterialValidation, Display, "Removing %d materials.", ToRemove.Num());

	for (FString const& Material : ToRemove)
	{
		UE_LOGF(LogMaterialValidation, Log, "Removing material '%ls' from '%ls'.", *Material, *InGroup->GetName());
		InGroup->Materials.Remove(Material);
	}

	if (ToRemove.Num() > 0)
	{
		InGroup->Modify();
	}
}

void UMaterialValidationLibrary::UpdateMaterialPermutationsInGroup(UMaterialValidationGroup* InGroup)
{
	if (InGroup == nullptr)
	{
		return;
	}
	
	UE_LOGF(LogMaterialValidation, Display, "Updating %d materials.", InGroup->Materials.Num());

	TArray<FString> SearchPaths;
	TArray<FString> ExcludePaths;
	MaterialValidationLibrary::GetSearchPaths(*InGroup, SearchPaths, ExcludePaths);

	FScopedSlowTask SlowTask(static_cast<float>(InGroup->Materials.Num()), LOCTEXT("UpdatingMaterials", "Updating Materials..."));
	
	for (TPair<FString, FMaterialValidationDesc>& It : InGroup->Materials)
	{
		SlowTask.EnterProgressFrame();

		TSoftObjectPtr<UMaterial> MaterialSoftPtr(MaterialValidationLibrary::ConvertStringPathToSoftObjectPath(It.Key));
		if (UMaterial const* Material = MaterialSoftPtr.LoadSynchronous())
		{
			MaterialValidationLibrary::UpdateValidMaterialInstances(*Material, ExcludePaths, It.Value);
		}
	}

	InGroup->Modify();
}

FSoftObjectPath UMaterialValidationLibrary::ResolveAssetPath(FString const& InAssetPath)
{
	return MaterialValidationLibrary::ConvertStringPathToSoftObjectPath(InAssetPath);
}

void UMaterialValidationLibrary::IsMaterialInGroup(UMaterialValidationGroup const* InGroup, UMaterial* InMaterial, bool& bOutIsInGroupPath, bool& bOutIsInGroup)
{
	bOutIsInGroupPath = false;
	bOutIsInGroup = false;

	if (InGroup == nullptr || InMaterial == nullptr)
	{
		return;
	}

	const FSoftObjectPath SoftPackagePath(InMaterial);
	if (MaterialValidationLibrary::FindMaterialDescFromSoftObjectPath(*InGroup, SoftPackagePath) != nullptr)
	{
		bOutIsInGroup = true;
	}

	TArray<FString> SearchPaths;
	TArray<FString> ExcludePaths;
	MaterialValidationLibrary::GetSearchPaths(*InGroup, SearchPaths, ExcludePaths);

	const FString PackagePath = SoftPackagePath.ToString();
	if (MaterialValidationLibrary::IsPackageInPaths(PackagePath, SearchPaths))
	{
		bOutIsInGroupPath = true;
	}

	if (bOutIsInGroupPath && MaterialValidationLibrary::IsPackageInPaths(PackagePath, ExcludePaths))
	{
		bOutIsInGroupPath = false;
	}
}

void UMaterialValidationLibrary::IsMaterialInstanceInGroup(UMaterialValidationGroup const* InGroup, UMaterialInstanceConstant* InMaterialInstance, bool& bOutMaterialInGroup, bool& bOutMaterialPermutationInGroup)
{
	bOutMaterialInGroup = false;
	bOutMaterialPermutationInGroup = false;

	if (InGroup == nullptr || InMaterialInstance == nullptr)
	{
		return;
	}

	UMaterial const* Material = InMaterialInstance->GetMaterial();
	const FSoftObjectPath SoftPackagePath(Material);

	FMaterialValidationDesc const* MaterialDesc = MaterialValidationLibrary::FindMaterialDescFromSoftObjectPath(*InGroup, SoftPackagePath);
	if (MaterialDesc == nullptr)
	{
		return;
	}

	bOutMaterialInGroup = true;

	const uint32 Hash = MaterialValidationLibrary::BuildPermutationHash(*InMaterialInstance, *Material);
	bOutMaterialPermutationInGroup = Hash == MaterialDesc->AssetData.PermutationHash || MaterialDesc->PermutationHashes.Contains(Hash);
}

int32 UMaterialValidationLibrary::GetShaderCount(UMaterialValidationGroup const* InGroup, UMaterial* InBaseMaterial)
{
	if (InGroup == nullptr || InBaseMaterial == nullptr)
	{
		return 0;
	}

	const FSoftObjectPath SoftPackagePath(InBaseMaterial);

	FMaterialValidationDesc const* MaterialDesc = MaterialValidationLibrary::FindMaterialDescFromSoftObjectPath(*InGroup, SoftPackagePath);
	return MaterialDesc != nullptr ? MaterialDesc->NumShadersTotal : 0;
}

int32 UMaterialValidationLibrary::GetModifiedShaderCount(
	UMaterialValidationGroup const* InGroup, 
	UMaterial* InBaseMaterial, 
	TArray<UMaterialInterface*> const& InModifiedObjects,
	TArray<UMaterialInterface*> const& InReplacementObjects,
	bool bForceLoadObjects)
{

	TRACE_CPUPROFILER_EVENT_SCOPE(UMaterialValidationLibrary::GetModifiedShaderCount)

	if (InGroup == nullptr || InBaseMaterial == nullptr)
	{
		return 0;
	}

	// Build a map from the canonical object path used in the database to the modified object entries.
	TMap<FSoftObjectPath, int32> ModifiedObjectPaths;
	for (int32 Index = 0; Index < InModifiedObjects.Num(); ++Index)
	{
		ModifiedObjectPaths.Add(FSoftObjectPath(InModifiedObjects[Index]), Index);
	}
	// Helper to find the modified entry from the path.
	auto GetModifiedObject = [&](FSoftObjectPath const& InPath) -> UMaterialInterface*
	{
		if (int32 const* FoundIndex = ModifiedObjectPaths.Find(InPath))
		{
			UMaterialInterface* ModifiedObject = InModifiedObjects.IsValidIndex(*FoundIndex) ? InModifiedObjects[*FoundIndex] : nullptr;
			UMaterialInterface* ReplacementObject = InReplacementObjects.IsValidIndex(*FoundIndex) ? InReplacementObjects[*FoundIndex] : nullptr;
			return ReplacementObject != nullptr ? ReplacementObject : ModifiedObject;
		}
		return nullptr;
	};

	UMaterial* BaseMaterial = InBaseMaterial;

	// Find any replacement for the base material.
	const int32 ModifiedObjectIndex = InModifiedObjects.Find(InBaseMaterial);
	const bool bIsBaseMaterialInModified = ModifiedObjectIndex != INDEX_NONE;
	if (InReplacementObjects.IsValidIndex(ModifiedObjectIndex))
	{
		UMaterial* Replacement = Cast<UMaterial>(InReplacementObjects[ModifiedObjectIndex]);
		BaseMaterial = Replacement != nullptr ? Replacement : BaseMaterial;
	}

	TArray<FString> SearchPaths;
	TArray<FString> ExcludePaths;
	MaterialValidationLibrary::GetSearchPaths(*InGroup, SearchPaths, ExcludePaths);

	MaterialValidationLibrary::FMaterialPermutationData PermutationData;
	BuildPermutationData(*BaseMaterial, PermutationData);

	FMaterialValidationAssetData_Material MaterialAssetDataFromObject;
	GetAssetData(*BaseMaterial, PermutationData, MaterialAssetDataFromObject);

	// Get the material asset data stored on the group.
	// But evaluate from the material if the material is modified or if it is not found in the group.
	const FSoftObjectPath BaseMaterialSoftObjectPath(InBaseMaterial);
	FMaterialValidationDesc const* MaterialDesc = MaterialValidationLibrary::FindMaterialDescFromSoftObjectPath(*InGroup, BaseMaterialSoftObjectPath);

	FMaterialValidationAssetData_Material MaterialAssetData;
	if (MaterialDesc == nullptr || bIsBaseMaterialInModified || bForceLoadObjects)
	{
		MaterialAssetData = MaterialAssetDataFromObject;
	}
	else
	{
		MaterialAssetData = MaterialDesc->AssetData;
	}

	// If the base material static parameters have changed from those stored on the group then any other material instance 
	// asset data is no longer valid and we can't iterate the hierarchy without loading the material instances assets.
	// So instead we approximate the change by only looking at the shader delta on the base material.
	if (MaterialDesc != nullptr && MaterialDesc->AssetData.StaticPropertyLayoutHash != MaterialAssetDataFromObject.StaticPropertyLayoutHash)
	{
		const int32 NumShadersBase = MaterialValidationLibrary::GetNumShaderTypes(*BaseMaterial);
		return MaterialDesc->NumShadersTotal + (NumShadersBase - MaterialDesc->NumShadersBase) * (1 + MaterialDesc->PermutationHashes.Num());
	}

	// Get the material instance hierarchy from the asset registry.
	TArray<FName> PackageNames;
	TArray<int32> HierarchyDepths;
	MaterialValidationLibrary::GetMaterialInstanceHierarchy(*InBaseMaterial, ExcludePaths, PackageNames, HierarchyDepths);

	// Use a stack of asset data to walk the hierarchy with the asset data overrides applied at each level.
	TArray<FMaterialValidationAssetData_MaterialInstance> AssetDataStack;

	// Build the root asset data from the base material.
	{
		FMaterialValidationAssetData_MaterialInstance& Root = AssetDataStack.AddDefaulted_GetRef();
		Root.StaticSwitchOverrideValues = MaterialAssetData.StaticSwitchValues;
		Root.ComponentMaskOverrideValues = MaterialAssetData.ComponentMaskValues;
		Root.StaticProperties = MaterialAssetData.StaticProperties;
		Root.MaterialLayerHash = MaterialAssetData.MaterialLayerHash;
		Root.PermutationHash = MaterialAssetData.PermutationHash;
	}

	// Initialize the shader count and unique hash array.
	int32 NumShaders = MaterialValidationLibrary::GetNumShaderTypes(*BaseMaterial);
	TArray<uint32> PermutationHashes;
	PermutationHashes.Add(MaterialAssetData.PermutationHash);

	const bool bBaseHasMaterialLayers = BaseMaterial->GetCachedExpressionData().bHasMaterialLayers;
	// Walk the material instance hierarchy.
	for (int32 Index = 1; Index < PackageNames.Num(); ++Index)
	{
		const int32 CurrentDepth = HierarchyDepths[Index];
		FString PackagePath = MaterialValidationLibrary::AddPackagePathExtension(PackageNames[Index]);
		FSoftObjectPath SoftObjectPackagePath = MaterialValidationLibrary::ConvertStringPathToSoftObjectPath(PackagePath);
		UMaterialInstanceConstant* MaterialInstance = Cast<UMaterialInstanceConstant>(GetModifiedObject(SoftObjectPackagePath));
		
		if (MaterialInstance == nullptr && bForceLoadObjects)
		{
			MaterialInstance = TSoftObjectPtr<UMaterialInstanceConstant>(SoftObjectPackagePath).LoadSynchronous();
		}

		FMaterialValidationAssetData_MaterialInstance AssetData;
		const bool bValidAssetData = MaterialValidationLibrary::GetMaterialInstanceAssetData(PackagePath, MaterialInstance, MaterialDesc, PermutationData, AssetData);

		if (!bValidAssetData)
		{
			// Ignore material instance and it's children.
			while (Index < PackageNames.Num() - 1 && HierarchyDepths[Index + 1] > CurrentDepth)
			{
				++Index;
			}
			continue;
		}

		// Apply the current asset data overrides to the parent to get the final asset data for the child.
		AssetDataStack.SetNum(CurrentDepth + 1);
		FMaterialValidationAssetData_MaterialInstance& Child = AssetDataStack[CurrentDepth];
		FMaterialValidationAssetData_MaterialInstance const& Parent = AssetDataStack[CurrentDepth - 1];

		Child.StaticSwitchOverrideValues.SetNumZeroed(Parent.StaticSwitchOverrideValues.Num());
		Child.ComponentMaskOverrideValues.SetNumZeroed(Parent.ComponentMaskOverrideValues.Num());
		MaterialValidationLibrary::ApplyBitMaskOverrides(Parent.StaticSwitchOverrideValues, AssetData.StaticSwitchOverrideMask, AssetData.StaticSwitchOverrideValues, Child.StaticSwitchOverrideValues, Child.StaticSwitchOverrideMask);
		MaterialValidationLibrary::ApplyBitMaskOverrides(Parent.ComponentMaskOverrideValues, AssetData.ComponentMaskOverrideMask, AssetData.ComponentMaskOverrideValues, Child.ComponentMaskOverrideValues, Child.ComponentMaskOverrideMask);
		MaterialValidationLibrary::ApplyBasePropertyOverrides(Parent.StaticProperties, AssetData.StaticPropertyOverrideFlags, AssetData.StaticProperties, Child.StaticProperties, Child.StaticPropertyOverrideFlags);
		Child.MaterialLayerHash = AssetData.MaterialLayerHash ? AssetData.MaterialLayerHash : Parent.MaterialLayerHash;

		// Build the permutation hash and if this is a new permutation then add its shader count.
		Child.PermutationHash = MaterialValidation::BuildPermutationHash(Child.StaticProperties, Child.StaticSwitchOverrideValues, Child.ComponentMaskOverrideValues, Child.MaterialLayerHash);
		if (!PermutationHashes.Contains(Child.PermutationHash))
		{
			UMaterialInstanceConstant* DummyMI = MaterialValidationLibrary::CreateMaterialInstance(*BaseMaterial, Child);

			// Layers currently do not support the fast path
			if (bBaseHasMaterialLayers)
			{
				TArray<FDebugShaderTypeInfo> ShaderTypeInfos;
				DummyMI->GetShaderTypes(GMaxRHIShaderPlatform, nullptr, ShaderTypeInfos);
				const TArray<FDebugShaderInfo> ValidationShaderDebugInfo = UMaterialEditingLibrary::GetShaderInfoFromTypeInfo(ShaderTypeInfos);

				NumShaders += ValidationShaderDebugInfo.Num();
			}
			else
			{
				NumShaders += MaterialValidationLibrary::GetNumShaderTypes(*DummyMI);
			}

			PermutationHashes.Add(Child.PermutationHash);
		}
	}

	return NumShaders;
}

/**
 * Note there is quite a bit of duplication with GetModifiedShaderCount() in this function.
 * It might be good to consolidate on a single material instance visitor that can do arbitrary work?
 */
void UMaterialValidationLibrary::GetMaterialHierarchyInfo(UMaterialValidationGroup const* InGroup, FSoftObjectPath const& InBaseMaterialPath, FMaterialDatabaseAssetHierarchyInfo& OutInfo)
{
	OutInfo.MaterialPaths.Reset();
	OutInfo.MaterialAssetDatas.Reset();
	OutInfo.StaticSwitchNames.Reset();
	OutInfo.ComponentMaskNames.Reset();

	if (InGroup == nullptr)
	{
		return;
	}

	const TSoftObjectPtr<UMaterial> BaseMaterialSoftPtr(InBaseMaterialPath);
	UMaterial const* BaseMaterial = BaseMaterialSoftPtr.LoadSynchronous();
	if (BaseMaterial == nullptr)
	{
		return;
	}

	// Get the description stored in the database.
	FMaterialValidationDesc const* MaterialDesc = MaterialValidationLibrary::FindMaterialDescFromSoftObjectPath(*InGroup, InBaseMaterialPath);
	if (MaterialDesc == nullptr)
	{
		return;
	}
	const FMaterialValidationAssetData_Material MaterialAssetData = MaterialDesc->AssetData;

	// Get the description of the current state.
	MaterialValidationLibrary::FMaterialPermutationData PermutationData;
	BuildPermutationData(*BaseMaterial, PermutationData);

	FMaterialValidationAssetData_Material MaterialAssetDataFromObject;
	GetAssetData(*BaseMaterial, PermutationData, MaterialAssetDataFromObject);

	OutInfo.StaticSwitchNames.SetNum(MaterialAssetData.StaticSwitchNum);
	OutInfo.ComponentMaskNames.SetNum(MaterialAssetData.ComponentMaskNum);

	// Check if the current static parameters are the same as those in the database.
	// If not then we can't use the found current names (and we don't persist the names in the database).
	if (MaterialAssetDataFromObject.StaticPropertyLayoutHash == MaterialAssetData.StaticPropertyLayoutHash &&
		PermutationData.StaticSwitchParameterInfo.Num() == MaterialAssetData.StaticSwitchNum &&
		PermutationData.ComponentMaskParameterInfo.Num() == MaterialAssetData.ComponentMaskNum)
	{
		for (int32 Index = 0; Index < MaterialAssetData.StaticSwitchNum; ++Index)
		{
			OutInfo.StaticSwitchNames[Index] = PermutationData.StaticSwitchParameterInfo[Index].Name;
		}
		for (int32 Index = 0; Index < MaterialAssetData.ComponentMaskNum; ++Index)
		{
			OutInfo.ComponentMaskNames[Index] = PermutationData.ComponentMaskParameterInfo[Index].Name;
		}
	}
	else
	{
		for (int32 Index = 0; Index < MaterialAssetData.StaticSwitchNum; ++Index)
		{
			static FName BaseName("StaticSwitch");
			OutInfo.StaticSwitchNames[Index] = FName(BaseName, Index);
		}
		for (int32 Index = 0; Index < MaterialAssetData.ComponentMaskNum; ++Index)
		{
			static FName BaseName("ComponentMask");
			OutInfo.ComponentMaskNames[Index] = FName(BaseName, Index);
		}
	}

	// Get the material instance hierarchy from the asset registry.
	TArray<FString> SearchPaths;
	TArray<FString> ExcludePaths;
	MaterialValidationLibrary::GetSearchPaths(*InGroup, SearchPaths, ExcludePaths);

	TArray<FName> PackageNames;
	TArray<int32> HierarchyDepths;
	MaterialValidationLibrary::GetMaterialInstanceHierarchy(*BaseMaterial, ExcludePaths, PackageNames, HierarchyDepths);

	// Use a stack of asset data to walk the hierarchy with the asset data overrides applied at each level.
	TArray<FMaterialValidationAssetData_MaterialInstance> AssetDataStack;

	// Build the root asset data from the base material.
	{
		FMaterialValidationAssetData_MaterialInstance& Root = AssetDataStack.AddDefaulted_GetRef();
		Root.StaticSwitchOverrideValues = MaterialAssetData.StaticSwitchValues;
		Root.StaticSwitchOverrideMask.SetNumZeroed(MaterialAssetData.StaticSwitchValues.Num());
		Root.ComponentMaskOverrideValues = MaterialAssetData.ComponentMaskValues;
		Root.ComponentMaskOverrideMask.SetNumZeroed(MaterialAssetData.ComponentMaskValues.Num());
		Root.StaticProperties = MaterialAssetData.StaticProperties;
		Root.MaterialLayerHash = MaterialAssetData.MaterialLayerHash;
		Root.PermutationHash = MaterialAssetData.PermutationHash;

		OutInfo.MaterialPaths.Add(InBaseMaterialPath);
		OutInfo.MaterialAssetDatas.Add(Root);
	}

	OutInfo.MaterialPaths.Reserve(PackageNames.Num() + 1);
	OutInfo.MaterialAssetDatas.Reserve(PackageNames.Num() + 1);

	// Walk the material instance hierarchy.
	for (int32 Index = 1; Index < PackageNames.Num(); ++Index)
	{
		const int32 CurrentDepth = HierarchyDepths[Index];
		FString PackagePath = MaterialValidationLibrary::AddPackagePathExtension(PackageNames[Index]);

		bool bValidAssetData = false;
		FMaterialValidationAssetData_MaterialInstance AssetData;
		
		int32 AssetDataIndex = INDEX_NONE;
		if (MaterialValidationLibrary::FindMaterialInstanceIndexFromSoftObjectPath(*MaterialDesc, PackagePath, AssetDataIndex))
		{
			if (AssetDataIndex == INDEX_NONE)
			{
				// Set up a default asset data with the right sizes.
				const int32 StaticSwitchArrayNum = MaterialDesc->AssetData.StaticSwitchValues.Num();
				AssetData.StaticSwitchOverrideMask.SetNumZeroed(StaticSwitchArrayNum);
				AssetData.StaticSwitchOverrideValues.SetNumZeroed(StaticSwitchArrayNum);
				const int32 ComponentMaskArrayNum = MaterialDesc->AssetData.ComponentMaskValues.Num();
				AssetData.ComponentMaskOverrideMask.SetNumZeroed(ComponentMaskArrayNum);
				AssetData.ComponentMaskOverrideValues.SetNumZeroed(ComponentMaskArrayNum);
				bValidAssetData = true;
			}
			else if (MaterialDesc->MaterialInstanceAssetDatas.IsValidIndex(AssetDataIndex))
			{
				AssetData = MaterialDesc->MaterialInstanceAssetDatas[AssetDataIndex];
				bValidAssetData = true;
			}
		}
		
		if (!bValidAssetData)
		{
			// Ignore material instance and it's children.
			while (Index < PackageNames.Num() - 1 && HierarchyDepths[Index + 1] > CurrentDepth)
			{
				++Index;
			}
			continue;
		}

		// Apply the current asset data overrides to the parent to get the final asset data for the child.
		AssetDataStack.SetNum(CurrentDepth + 1);
		FMaterialValidationAssetData_MaterialInstance& Child = AssetDataStack[CurrentDepth];
		FMaterialValidationAssetData_MaterialInstance const& Parent = AssetDataStack[CurrentDepth - 1];

		Child.StaticSwitchOverrideValues.SetNumZeroed(Parent.StaticSwitchOverrideValues.Num());
		Child.ComponentMaskOverrideValues.SetNumZeroed(Parent.ComponentMaskOverrideValues.Num());
		MaterialValidationLibrary::ApplyBitMaskOverrides(Parent.StaticSwitchOverrideValues, AssetData.StaticSwitchOverrideMask, AssetData.StaticSwitchOverrideValues, Child.StaticSwitchOverrideValues, Child.StaticSwitchOverrideMask);
		MaterialValidationLibrary::ApplyBitMaskOverrides(Parent.ComponentMaskOverrideValues, AssetData.ComponentMaskOverrideMask, AssetData.ComponentMaskOverrideValues, Child.ComponentMaskOverrideValues, Child.ComponentMaskOverrideMask);
		MaterialValidationLibrary::ApplyBasePropertyOverrides(Parent.StaticProperties, AssetData.StaticPropertyOverrideFlags, AssetData.StaticProperties, Child.StaticProperties, Child.StaticPropertyOverrideFlags);
		Child.MaterialLayerHash = AssetData.MaterialLayerHash;
		Child.PermutationHash = MaterialValidation::BuildPermutationHash(Child.StaticProperties, Child.StaticSwitchOverrideValues, Child.ComponentMaskOverrideValues, Child.MaterialLayerHash);

		OutInfo.MaterialPaths.Add(MaterialValidationLibrary::ConvertStringPathToSoftObjectPath(PackagePath));
		OutInfo.MaterialAssetDatas.Add(Child);
	}
}

namespace MaterialValidationLibrary {

/**
 * Enumeration of all of the UMaterialInstance base properties. These are the property overrides that trigger static permutations.
 */
enum class EMaterialInstanceStaticProperties : int32
{
	TwoSided,
	bIsThinSurface,
	DitheredLODTransition,
	bCastDynamicShadowAsMasked,
	bOutputTranslucentVelocity,
	bHasPixelAnimation,
	bEnableTessellation,
	BlendMode,
	ShadingModel,
	OpacityMaskClipValue,
	Count
};

/** Helper getter functions for each of the EMaterialInstanceStaticProperties. */
struct FMaterialInstanceStaticPropertyGetters
{
	static uint8 GetTwoSided(FStaticPermutationProperties const& P) { return P.TwoSided; }
	static uint8 GetIsThinSurface(FStaticPermutationProperties const& P) { return P.bIsThinSurface; }
	static uint8 GetDitheredLODTransition(FStaticPermutationProperties const& P) { return P.DitheredLODTransition; }
	static uint8 GetCastDynamicShadowAsMasked(FStaticPermutationProperties const& P) { return P.bCastDynamicShadowAsMasked; }
	static uint8 GetOutputTranslucentVelocity(FStaticPermutationProperties const& P) { return P.bOutputTranslucentVelocity; }
	static uint8 GetHasPixelAnimation(FStaticPermutationProperties const& P) { return P.bHasPixelAnimation; }
	static uint8 GetEnableTessellation(FStaticPermutationProperties const& P) { return P.bEnableTessellation; }
	static EBlendMode GetBlendMode(FStaticPermutationProperties const& P) { return P.BlendMode; }
	static EMaterialShadingModel GetShadingModel(FStaticPermutationProperties const& P) { return P.ShadingModel; }
	static float GetOpacityMaskClipValue(FStaticPermutationProperties const& P) { return P.OpacityMaskClipValue; }
};

struct FMaterialInstanceStaticPropertyModifiedGetters
{
	static bool GetTwoSidedModified(FStaticPermutationPropertyOverrideFlags const& P) { return P.bOverride_TwoSided; }
	static bool GetIsThinSurfaceModified(FStaticPermutationPropertyOverrideFlags const& P) { return P.bOverride_bIsThinSurface; }
	static bool GetDitheredLODTransitionModified(FStaticPermutationPropertyOverrideFlags const& P) { return P.bOverride_DitheredLODTransition; }
	static bool GetCastDynamicShadowAsMaskedModified(FStaticPermutationPropertyOverrideFlags const& P) { return P.bOverride_CastDynamicShadowAsMasked; }
	static bool GetOutputTranslucentVelocityModified(FStaticPermutationPropertyOverrideFlags const& P) { return P.bOverride_OutputTranslucentVelocity; }
	static bool GetHasPixelAnimationModified(FStaticPermutationPropertyOverrideFlags const& P) { return P.bOverride_bHasPixelAnimation; }
	static bool GetEnableTessellationModified(FStaticPermutationPropertyOverrideFlags const& P) { return P.bOverride_bEnableTessellation; }
	static bool GetBlendModeModified(FStaticPermutationPropertyOverrideFlags const& P) { return P.bOverride_BlendMode; }
	static bool GetShadingModelModified(FStaticPermutationPropertyOverrideFlags const& P) { return P.bOverride_ShadingModel; }
	static bool GetOpacityMaskClipModified(FStaticPermutationPropertyOverrideFlags const& P) { return P.bOverride_OpacityMaskClipValue; }
};

template <auto GetterPtr>
static FString GetStaticPropertyValueAsString(FStaticPermutationProperties const& InProperties)
{
	using ReturnType = TInvokeResult_T<decltype(GetterPtr), FStaticPermutationProperties const&>;

	if constexpr (TIsEnum<ReturnType>::Value)
	{
		return  UEnum::GetDisplayValueAsText((*GetterPtr)(InProperties)).ToString();
	}
 	else if constexpr (TIsFloatingPoint<ReturnType>::Value)
 	{
 		return FString::Printf(TEXT("%g"), (*GetterPtr)(InProperties));
 	}
	else
	{
		return LexToString((*GetterPtr)(InProperties));
	}
}

using FnGetPropertyValueAsString = FString(*)(FStaticPermutationProperties const&);
using FnIsPropertyModified = bool(*)(FStaticPermutationPropertyOverrideFlags const&);

/** Struct containing information needed to generate table entries for EMaterialInstanceStaticProperties. */
struct FMaterialInstanceStaticPropertyInfo
{
	FName PropertyId;
	FName PropertyName;
	FnGetPropertyValueAsString GetPropertyValueAsStringFunction;
	FnIsPropertyModified GetIsPropertyModifiedFunction;
};

/** Array for table generation data for EMaterialInstanceStaticProperties. */
static const FMaterialInstanceStaticPropertyInfo StaticPropertyInfos[(int32)EMaterialInstanceStaticProperties::Count] = {
	{"PID_TwoSided", "Two Sided", &GetStaticPropertyValueAsString <&FMaterialInstanceStaticPropertyGetters::GetTwoSided>, &FMaterialInstanceStaticPropertyModifiedGetters::GetTwoSidedModified },
	{"PID_IsThinSurface", "Thin Surface", &GetStaticPropertyValueAsString <&FMaterialInstanceStaticPropertyGetters::GetIsThinSurface>, &FMaterialInstanceStaticPropertyModifiedGetters::GetIsThinSurfaceModified },
	{"PID_DitheredLODTransition", "Dithered LOD Transition", &GetStaticPropertyValueAsString <&FMaterialInstanceStaticPropertyGetters::GetDitheredLODTransition>, &FMaterialInstanceStaticPropertyModifiedGetters::GetDitheredLODTransitionModified },
	{"PID_CastDynamicShadowAsMasked", "Dynamic Shadow As Masked", &GetStaticPropertyValueAsString <&FMaterialInstanceStaticPropertyGetters::GetCastDynamicShadowAsMasked>, &FMaterialInstanceStaticPropertyModifiedGetters::GetCastDynamicShadowAsMaskedModified },
	{"PID_OutputTranslucentVelocity", "Output Translucent Velocity", &GetStaticPropertyValueAsString <&FMaterialInstanceStaticPropertyGetters::GetOutputTranslucentVelocity>, &FMaterialInstanceStaticPropertyModifiedGetters::GetOutputTranslucentVelocityModified },
	{"PID_HasPixelAnimation", "Has Pixel Animation", &GetStaticPropertyValueAsString <&FMaterialInstanceStaticPropertyGetters::GetHasPixelAnimation>, &FMaterialInstanceStaticPropertyModifiedGetters::GetHasPixelAnimationModified },
	{"PID_EnableTessellation", "Tessellation", &GetStaticPropertyValueAsString <&FMaterialInstanceStaticPropertyGetters::GetEnableTessellation>, &FMaterialInstanceStaticPropertyModifiedGetters::GetEnableTessellationModified },
	{"PID_BlendMode", "Blend Mode", &GetStaticPropertyValueAsString <&FMaterialInstanceStaticPropertyGetters::GetBlendMode>, &FMaterialInstanceStaticPropertyModifiedGetters::GetBlendModeModified },
	{"PID_ShadingModel", "Shading Model", &GetStaticPropertyValueAsString <&FMaterialInstanceStaticPropertyGetters::GetShadingModel>, &FMaterialInstanceStaticPropertyModifiedGetters::GetShadingModelModified },
	{"PID_OpacityMaskClipValue", "Opacity Mask Clip Value", &GetStaticPropertyValueAsString <&FMaterialInstanceStaticPropertyGetters::GetOpacityMaskClipValue>, &FMaterialInstanceStaticPropertyModifiedGetters::GetOpacityMaskClipModified },
};

/** Static array of column FNames for all material usages. */
static const FName UsageFlagPropertyIds[] = {
	"PID_UsageSkeletalMesh",
	"PID_UsageParticleSprites",
	"PID_UsageBeamTrails",
	"PID_UsageMeshParticles",
	"PID_UsageStaticLighting",
	"PID_UsageMorphTargets",
	"PID_UsageSplineMesh",
	"PID_UsageInstancedStaticMeshes",
	"PID_UsageGeometryCollections",
	"PID_UsageClothing",
	"PID_UsageNiagaraSprites",
	"PID_UsageNiagaraRibbons",
	"PID_UsageNiagaraMeshParticles",
	"PID_UsageGeometryCache",
	"PID_UsageWater",
	"PID_UsageHairStrands",
	"PID_UsageLidarPointCloud",
	"PID_UsageVirtualHeightfieldMesh",
	"PID_UsageNanite",
	"PID_UsageVoxels",
	"PID_UsageCurves",
	"PID_UsageVolumetricCloud",
	"PID_UsageHeterogeneousVolumes",
	"PID_UsageStaticMesh",
	"PID_UsageEditorCompositing",
	"PID_UsageNeuralNetworks",
	"PID_UsageMeshDeformer",
	"PID_UsageInstancedSkinnedMesh",
};
static_assert(UE_ARRAY_COUNT(UsageFlagPropertyIds) == MATUSAGE_MAX, "UsageFlagPropertyIds must match EMaterialUsage. Did you add a new MATUSAGE entry?");

static int32 GetPropertyCount(FMaterialDatabaseAssetHierarchyInfo const& InInfo)
{
	return UE_ARRAY_COUNT(StaticPropertyInfos) + MATUSAGE_MAX + InInfo.StaticSwitchNames.Num() + InInfo.ComponentMaskNames.Num();
}

/** Read a single bit from a packed uint32 array (1 bit per element). Returns false if the index is out of range. */
static bool GetPackedBit(TConstArrayView<uint32> InArray, int32 Index)
{
	const uint32 BitArrayIndex = (uint32)Index / 32;
	const uint32 BitIndex = (uint32)Index % 32;
	return InArray.IsValidIndex(BitArrayIndex) && (InArray[BitArrayIndex] & (1u << BitIndex)) != 0;
}

/** Read a nibble (4 bits) from a packed uint32 array (4 bits per element). Returns 0 if the index is out of range. */
static uint32 GetPackedNibble(TConstArrayView<uint32> InArray, int32 Index)
{
	const uint32 BitArrayIndex = Index / 8;
	const uint32 BitIndex = (Index * 4) % 32;
	return InArray.IsValidIndex(BitArrayIndex) ? (InArray[BitArrayIndex] >> BitIndex) & 0xf : 0u;
}

/** Format a component mask nibble as an RGBA_ string. */
static FString FormatComponentMask(uint32 InValue)
{
	TStringBuilder<8> RGBA;
	RGBA.AppendChar(InValue & 1 ? 'R' : '_');
	RGBA.AppendChar(InValue & 2 ? 'G' : '_');
	RGBA.AppendChar(InValue & 4 ? 'B' : '_');
	RGBA.AppendChar(InValue & 8 ? 'A' : '_');
	return *RGBA;
}


/** Get all the material property values for a material instance in a material hierarchy. */
static void GetPropertyValuesFromAssetData(
	FMaterialDatabaseAssetHierarchyInfo const& InInfo,
	FMaterialValidationAssetData_MaterialInstance const& AssetData,
	TArray<FMaterialDatabaseAssetPropertyValue>& OutValues)
{
	const int32 PropertyCount = MaterialValidationLibrary::GetPropertyCount(InInfo);
	OutValues.Reset(PropertyCount);

	for (FMaterialInstanceStaticPropertyInfo const& PropInfo : StaticPropertyInfos)
	{
		OutValues.Add({
			.Value = PropInfo.GetPropertyValueAsStringFunction(AssetData.StaticProperties),
			.bIsModifiedFromParent = PropInfo.GetIsPropertyModifiedFunction(AssetData.StaticPropertyOverrideFlags) });
	}

	for (int32 Index = 0; Index < MATUSAGE_MAX; ++Index)
	{
		const bool bValue = (AssetData.StaticProperties.UsageFlags & (1u << Index)) != 0;
		const bool bModified = (AssetData.StaticPropertyOverrideFlags.Override_UsageFlags & (1u << Index)) != 0;
		OutValues.Add({ .Value = bValue ? TEXT("1") : TEXT("0"), .bIsModifiedFromParent = bModified });
	}

	for (int32 Index = 0; Index < InInfo.StaticSwitchNames.Num(); ++Index)
	{
		const uint32 BitArrayIndex = (uint32)Index / 32;
		const uint32 BitIndex = (uint32)Index % 32;

		if (ensure(AssetData.StaticSwitchOverrideValues.IsValidIndex(BitArrayIndex) && AssetData.StaticSwitchOverrideMask.IsValidIndex(BitArrayIndex)))
		{
			const bool bValue = (AssetData.StaticSwitchOverrideValues[BitArrayIndex] & (1u << BitIndex)) != 0;
			const bool bModified = (AssetData.StaticSwitchOverrideMask[BitArrayIndex] & (1u << BitIndex)) != 0;
			OutValues.Add({ .Value = bValue ? TEXT("1") : TEXT("0"), .bIsModifiedFromParent = bModified });
		}
		else
		{
			// Mismatch possibly caused by add/removal of parameters since database build.
			OutValues.Add({ .Value = TEXT("?"), .bIsModifiedFromParent = false });
		}
	}

	for (int32 Index = 0; Index < InInfo.ComponentMaskNames.Num(); ++Index)
	{
		const uint32 BitArrayIndex = Index / 8;
		const uint32 BitIndex = (Index * 4) % 32;

		if (ensure(AssetData.ComponentMaskOverrideValues.IsValidIndex(BitArrayIndex) && AssetData.ComponentMaskOverrideMask.IsValidIndex(BitArrayIndex)))
		{
			const uint32 Value = (AssetData.ComponentMaskOverrideValues[BitArrayIndex] >> BitIndex) & 0xf;
			const bool bModified = ((AssetData.ComponentMaskOverrideMask[BitArrayIndex] >> BitIndex) & 0xf) != 0;
			OutValues.Add({ .Value = MaterialValidationLibrary::FormatComponentMask(Value), .bIsModifiedFromParent = bModified });
		}
		else
		{
			// Mismatch possibly caused by add/removal of parameters since database build.
			OutValues.Add({ .Value = TEXT("?"), .bIsModifiedFromParent = false });
		}
	}
}

} // namespace MaterialValidationLibrary

void UMaterialValidationLibrary::GetMaterialProperties(FMaterialDatabaseAssetHierarchyInfo const& InInfo, TArray<FMaterialDatabaseAssetPropertyDesc>& OutProperties)
{
	const int32 PropertyCount = MaterialValidationLibrary::GetPropertyCount(InInfo);
	OutProperties.Reset(PropertyCount);

	for (MaterialValidationLibrary::FMaterialInstanceStaticPropertyInfo const& StaticPropertyInfo : MaterialValidationLibrary::StaticPropertyInfos)
	{
		OutProperties.Add({ .Id = StaticPropertyInfo.PropertyId, .Name = StaticPropertyInfo.PropertyName, .Category = EMaterialPropertyCategory::StaticProperty });
	}
	for (int32 Index = 0; Index < MATUSAGE_MAX; ++Index)
	{
		OutProperties.Add({ .Id = MaterialValidationLibrary::UsageFlagPropertyIds[Index], .Name = FName(*UMaterialInterface::GetUsageName((EMaterialUsage)Index)), .Category = EMaterialPropertyCategory::UsageFlag});
	}
	for (const FName Name : InInfo.StaticSwitchNames)
	{
		OutProperties.Add({ .Id = Name, .Name = Name, .Category = EMaterialPropertyCategory::StaticSwitch});
	}
	for (const FName Name : InInfo.ComponentMaskNames)
	{
		OutProperties.Add({ .Id = Name, .Name = Name, .Category = EMaterialPropertyCategory::ComponentMask });
	}
}
	
void UMaterialValidationLibrary::GetMaterialPropertyValues(FMaterialDatabaseAssetHierarchyInfo const& InInfo, int32 InMaterialIndex, TArray<FMaterialDatabaseAssetPropertyValue>& OutValues)
{
	if (!ensure(InInfo.MaterialAssetDatas.IsValidIndex(InMaterialIndex)))
	{
		OutValues.Reset();
		return;
	}

	MaterialValidationLibrary::GetPropertyValuesFromAssetData(InInfo, InInfo.MaterialAssetDatas[InMaterialIndex], OutValues);
}

void UMaterialValidationLibrary::GetMaterialPropertyValues(FMaterialDatabaseAssetHierarchyInfo const& InInfo, UMaterialInstanceConstant* InMaterialInstance, TArray<FMaterialDatabaseAssetPropertyValue>& OutValues)
{
	OutValues.Reset();

	if (InMaterialInstance == nullptr)
	{
		return;
	}

	UMaterial* const BaseMaterial = InMaterialInstance->GetMaterial();
	if (BaseMaterial == nullptr)
	{
		return;
	}

	MaterialValidationLibrary::FMaterialPermutationData PermutationData;
	MaterialValidationLibrary::BuildPermutationData(*BaseMaterial, PermutationData);

	FMaterialValidationAssetData_MaterialInstance AssetData;
	MaterialValidationLibrary::GetAssetData(*InMaterialInstance, PermutationData, AssetData);

	MaterialValidationLibrary::GetPropertyValuesFromAssetData(InInfo, AssetData, OutValues);
}

namespace MaterialValidationLibrary {

/** String shown to indicate that a material instance property is being inherited from its parent. */
static const TCHAR* GInheritedValueString = TEXT("[Inherited]");

/** Add an entry to InOutDiffs if old and new string values differ. */
static void AddDiffIfChanged(FName InName, EMaterialPropertyCategory InCategory, FString const& InOldValue, FString const& InNewValue, TArray<FMaterialInstancePropertyDiff>& InOutDiffs)
{
	if (InOldValue != InNewValue)
	{
		InOutDiffs.Add({ .PropertyName = InName, .Category = InCategory, .OldValue = InOldValue, .NewValue = InNewValue });
	}
}

/** Diff all static permutation properties between two base materials (no inheritance). */
static void BuildStaticPropertyDiff(
	FStaticPermutationProperties const& InOld,
	FStaticPermutationProperties const& InNew,
	TArray<FMaterialInstancePropertyDiff>& InOutDiffs)
{
	for (MaterialValidationLibrary::FMaterialInstanceStaticPropertyInfo const& StaticPropertyInfo : StaticPropertyInfos)
	{
		AddDiffIfChanged(
			StaticPropertyInfo.PropertyName,
			EMaterialPropertyCategory::StaticProperty,
			StaticPropertyInfo.GetPropertyValueAsStringFunction(InOld),
			StaticPropertyInfo.GetPropertyValueAsStringFunction(InNew),
			InOutDiffs);
	}
}

/** Diff all static permutation properties between two material instances, showing "[Inherited]" for non-overridden properties. */
static void BuildStaticPropertyDiff(
	FStaticPermutationProperties const& InOld,
	FStaticPermutationProperties const& InNew,
	FStaticPermutationPropertyOverrideFlags const& InOldOverrideFlags,
	FStaticPermutationPropertyOverrideFlags const& InNewOverrideFlags,
	TArray<FMaterialInstancePropertyDiff>& InOutDiffs)
{
	for (MaterialValidationLibrary::FMaterialInstanceStaticPropertyInfo const& StaticPropertyInfo : StaticPropertyInfos)
	{
		const bool bOldInherited = !StaticPropertyInfo.GetIsPropertyModifiedFunction(InOldOverrideFlags);
		const bool bNewInherited = !StaticPropertyInfo.GetIsPropertyModifiedFunction(InNewOverrideFlags);
		AddDiffIfChanged(
			StaticPropertyInfo.PropertyName,
			EMaterialPropertyCategory::StaticProperty,
			bOldInherited ? GInheritedValueString : StaticPropertyInfo.GetPropertyValueAsStringFunction(InOld),
			bNewInherited ? GInheritedValueString : StaticPropertyInfo.GetPropertyValueAsStringFunction(InNew),
			InOutDiffs);
	}
}

/** Diff all usage flags between two base materials (no inheritance). */
static void BuildUsageFlagDiff(
	FStaticPermutationProperties const& InOld,
	FStaticPermutationProperties const& InNew,
	TArray<FMaterialInstancePropertyDiff>& InOutDiffs)
{
	for (int32 Index = 0; Index < MATUSAGE_MAX; ++Index)
	{
		const FName UsageName(*UMaterialInterface::GetUsageName((EMaterialUsage)Index));
		const FString OldValue = (InOld.UsageFlags & (1u << Index)) != 0 ? TEXT("1") : TEXT("0");
		const FString NewValue = (InNew.UsageFlags & (1u << Index)) != 0 ? TEXT("1") : TEXT("0");
		AddDiffIfChanged(UsageName, EMaterialPropertyCategory::UsageFlag, OldValue, NewValue, InOutDiffs);
	}
}

/** Diff all usage flags between two material instances, showing "[Inherited]" for non-overridden flags. */
static void BuildUsageFlagDiff(
	FStaticPermutationProperties const& InOld,
	FStaticPermutationProperties const& InNew,
	FStaticPermutationPropertyOverrideFlags const& InOldOverrideFlags,
	FStaticPermutationPropertyOverrideFlags const& InNewOverrideFlags,
	TArray<FMaterialInstancePropertyDiff>& InOutDiffs)
{
	for (int32 Index = 0; Index < MATUSAGE_MAX; ++Index)
	{
		const FName UsageName(*UMaterialInterface::GetUsageName((EMaterialUsage)Index));
		const FString OldValue = !(InOldOverrideFlags.Override_UsageFlags & (1u << Index)) ? GInheritedValueString : (InOld.UsageFlags & (1u << Index)) != 0 ? TEXT("1") : TEXT("0");
		const FString NewValue = !(InNewOverrideFlags.Override_UsageFlags & (1u << Index)) ? GInheritedValueString : (InNew.UsageFlags & (1u << Index)) != 0 ? TEXT("1") : TEXT("0");
		AddDiffIfChanged(UsageName, EMaterialPropertyCategory::UsageFlag, OldValue, NewValue, InOutDiffs);
	}
}

/** Diff all static switch parameters between two base materials (no inheritance). */
static void BuildStaticSwitchDiff(
	TConstArrayView<uint32> InOld,
	TConstArrayView<uint32> InNew,
	TConstArrayView<FName> InStaticSwitchNames,
	TArray<FMaterialInstancePropertyDiff>& InOutDiffs)
{
	for (int32 Index = 0; Index < InStaticSwitchNames.Num(); ++Index)
	{
		const FString OldValue = GetPackedBit(InOld, Index) ? TEXT("1") : TEXT("0");
		const FString NewValue = GetPackedBit(InNew, Index) ? TEXT("1") : TEXT("0");
		AddDiffIfChanged(InStaticSwitchNames[Index], EMaterialPropertyCategory::StaticSwitch, OldValue, NewValue, InOutDiffs);
	}
}

/** Diff all static switch parameters between two material instances, showing "[Inherited]" for non-overridden switches. */
static void BuildStaticSwitchDiff(
	TConstArrayView<uint32> InOld,
	TConstArrayView<uint32> InNew,
	TConstArrayView<uint32> InOldOverrideMask,
	TConstArrayView<uint32> InNewOverrideMask,
	TConstArrayView<FName> InStaticSwitchNames,
	TArray<FMaterialInstancePropertyDiff>& InOutDiffs)
{
	for (int32 Index = 0; Index < InStaticSwitchNames.Num(); ++Index)
	{
		const FString OldValue = !GetPackedBit(InOldOverrideMask, Index) ? GInheritedValueString : GetPackedBit(InOld, Index) ? TEXT("1") : TEXT("0");
		const FString NewValue = !GetPackedBit(InNewOverrideMask, Index) ? GInheritedValueString : GetPackedBit(InNew, Index) ? TEXT("1") : TEXT("0");
		AddDiffIfChanged(InStaticSwitchNames[Index], EMaterialPropertyCategory::StaticSwitch, OldValue, NewValue, InOutDiffs);
	}
}

/** Diff all component mask parameters between two base materials (no inheritance). */
static void BuildComponentMaskDiff(
	TConstArrayView<uint32> InOld,
	TConstArrayView<uint32> InNew,
	TConstArrayView<FName> InComponentMaskNames,
	TArray<FMaterialInstancePropertyDiff>& InOutDiffs)
{
	for (int32 Index = 0; Index < InComponentMaskNames.Num(); ++Index)
	{
		const FString OldValue = FormatComponentMask(GetPackedNibble(InOld, Index));
		const FString NewValue = FormatComponentMask(GetPackedNibble(InNew, Index));
		AddDiffIfChanged(InComponentMaskNames[Index], EMaterialPropertyCategory::ComponentMask, OldValue, NewValue, InOutDiffs);
	}
}

/** Diff all component mask parameters between two material instances, showing "[Inherited]" for non-overridden masks. */
static void BuildComponentMaskDiff(
	TConstArrayView<uint32> InOld,
	TConstArrayView<uint32> InNew,
	TConstArrayView<uint32> InOldOverrideMask,
	TConstArrayView<uint32> InNewOverrideMask,
	TConstArrayView<FName> InComponentMaskNames,
	TArray<FMaterialInstancePropertyDiff>& InOutDiffs)
{
	for (int32 Index = 0; Index < InComponentMaskNames.Num(); ++Index)
	{
		const FString OldValue = !GetPackedNibble(InOldOverrideMask, Index) ? GInheritedValueString : FormatComponentMask(GetPackedNibble(InOld, Index));
		const FString NewValue = !GetPackedNibble(InNewOverrideMask, Index) ? GInheritedValueString : FormatComponentMask(GetPackedNibble(InNew, Index));
		AddDiffIfChanged(InComponentMaskNames[Index], EMaterialPropertyCategory::ComponentMask, OldValue, NewValue, InOutDiffs);
	}
}

/** Build property diffs for a base material entry. */
static void BuildBaseMaterialPropertyDiff(
	FMaterialValidationAssetData_Material const& InOldData,
	FMaterialValidationAssetData_Material const& InNewData,
	TConstArrayView<FName> InStaticSwitchNames,
	TConstArrayView<FName> InComponentMaskNames,
	TArray<FMaterialInstancePropertyDiff>& OutDiffs)
{
	BuildStaticPropertyDiff(InOldData.StaticProperties, InNewData.StaticProperties, OutDiffs);
	BuildUsageFlagDiff(InOldData.StaticProperties, InNewData.StaticProperties, OutDiffs);

	// Switch parameters are only comparable when layout hashes match.
	if (InOldData.StaticPropertyLayoutHash == InNewData.StaticPropertyLayoutHash)
	{
		BuildStaticSwitchDiff(InOldData.StaticSwitchValues, InNewData.StaticSwitchValues, InStaticSwitchNames, OutDiffs);
		BuildComponentMaskDiff(InOldData.ComponentMaskValues, InNewData.ComponentMaskValues, InComponentMaskNames, OutDiffs);
	}
	else
	{
		OutDiffs.Add({
			.PropertyName = "Static Parameter Layout",
			.OldValue = FString::Printf(TEXT("0x%08X"), InOldData.StaticPropertyLayoutHash),
			.NewValue = FString::Printf(TEXT("0x%08X"), InNewData.StaticPropertyLayoutHash) });
	}

	// Material layer hash.
	if (InOldData.MaterialLayerHash != InNewData.MaterialLayerHash)
	{
		OutDiffs.Add({
			.PropertyName = "Material Layer",
			.OldValue = FString::Printf(TEXT("0x%08X"), InOldData.MaterialLayerHash),
			.NewValue = FString::Printf(TEXT("0x%08X"), InNewData.MaterialLayerHash) });
	}
}

/**
 * Build property diffs for a material instance entry.
 * Compares raw override data and returns a value of "[Inherited]" when not overriding.
 */
static void BuildInstancePropertyDiff(
	FMaterialValidationAssetData_MaterialInstance const& InOldData,
	FMaterialValidationAssetData_MaterialInstance const& InNewData,
	TConstArrayView<FName> InStaticSwitchNames,
	TConstArrayView<FName> InComponentMaskNames,
	bool bLayoutsMatch,
	TArray<FMaterialInstancePropertyDiff>& OutDiffs)
{
	// Static permutation properties and usage flags with override flags.
	BuildStaticPropertyDiff(InOldData.StaticProperties, InNewData.StaticProperties, InOldData.StaticPropertyOverrideFlags, InNewData.StaticPropertyOverrideFlags, OutDiffs);
	BuildUsageFlagDiff(InOldData.StaticProperties, InNewData.StaticProperties, InOldData.StaticPropertyOverrideFlags, InNewData.StaticPropertyOverrideFlags, OutDiffs);

	// Switch parameters are only comparable when layout hashes match.
	if (bLayoutsMatch)
	{
		BuildStaticSwitchDiff(InOldData.StaticSwitchOverrideValues, InNewData.StaticSwitchOverrideValues, InOldData.StaticSwitchOverrideMask, InNewData.StaticSwitchOverrideMask, InStaticSwitchNames, OutDiffs);
		BuildComponentMaskDiff(InOldData.ComponentMaskOverrideValues, InNewData.ComponentMaskOverrideValues, InOldData.ComponentMaskOverrideMask, InNewData.ComponentMaskOverrideMask, InComponentMaskNames, OutDiffs);
	}

	// Material layer hash.
	if (InOldData.MaterialLayerHash != InNewData.MaterialLayerHash)
	{
		OutDiffs.Add({ .PropertyName = "Material Layer",
			.OldValue = FString::Printf(TEXT("0x%08X"), InOldData.MaterialLayerHash),
			.NewValue = FString::Printf(TEXT("0x%08X"), InNewData.MaterialLayerHash) });
	}
}

} // namespace MaterialValidationLibrary

void UMaterialValidationLibrary::GetMaterialValidationDescDiff(
	UMaterialValidationGroup const* InGroupOld,
	UMaterialValidationGroup const* InGroupNew,
	FSoftObjectPath const& InBaseMaterialPath,
	bool bAllowAssetLoad,
	TArray<FMaterialInstanceDiffResult>& OutDiffs)
{
	OutDiffs.Reset();

	FMaterialValidationDesc const* OldDesc = InGroupOld != nullptr ? MaterialValidationLibrary::FindMaterialDescFromSoftObjectPath(*InGroupOld, InBaseMaterialPath) : nullptr;
	FMaterialValidationDesc const* NewDesc = InGroupNew != nullptr ? MaterialValidationLibrary::FindMaterialDescFromSoftObjectPath(*InGroupNew, InBaseMaterialPath) : nullptr;

	if (OldDesc == nullptr && NewDesc == nullptr)
	{
		return;
	}
	else if (OldDesc == nullptr && NewDesc != nullptr)
	{
		OutDiffs.Add({ .InstancePath = InBaseMaterialPath, .DiffType = EMaterialInstanceDiffType::Added });
		return;
	}
	else if (OldDesc != nullptr && NewDesc == nullptr)
	{
		OutDiffs.Add({ .InstancePath = InBaseMaterialPath, .DiffType = EMaterialInstanceDiffType::Removed });
		return;
	}

	// Resolve switch and component mask parameter names if allowed.
	TArray<FName> SwitchNames;
	TArray<FName> ComponentMaskNames;

	TSoftObjectPtr<UMaterial> BaseMaterialSoftPtr(InBaseMaterialPath);
	bool bFoundParameterNames = false;
	if (bAllowAssetLoad || BaseMaterialSoftPtr.IsValid())
	{
		if (UMaterial const* BaseMaterial = BaseMaterialSoftPtr.LoadSynchronous())
		{
			MaterialValidationLibrary::FMaterialPermutationData PermutationData;
			MaterialValidationLibrary::BuildPermutationData(*BaseMaterial, PermutationData);

			// Only use resolved names if the layout matches the new snapshot.
			if (PermutationData.StaticPropertyLayoutHash == NewDesc->AssetData.StaticPropertyLayoutHash)
			{
				for (FMaterialParameterInfo const& Info : PermutationData.StaticSwitchParameterInfo)
				{
					SwitchNames.Add(Info.Name);
				}
				for (FMaterialParameterInfo const& Info : PermutationData.ComponentMaskParameterInfo)
				{
					ComponentMaskNames.Add(Info.Name);
				}
			
				bFoundParameterNames = true;
			}
		}
	}

	// Use default parameter names as a fallback.
	if (!bFoundParameterNames)
	{
		{
			static FName BaseName("StaticSwitch");
			const int32 MaxSwitchNum = FMath::Max(OldDesc->AssetData.StaticSwitchNum, NewDesc->AssetData.StaticSwitchNum);
			for (int32 Index = 0; Index < MaxSwitchNum; ++Index)
			{
				SwitchNames.Add(FName(BaseName, Index));
			}
		}
		{
			static const FName BaseName("ComponentMask");
			const int32 MaxComponentMaskNum = FMath::Max(OldDesc->AssetData.ComponentMaskNum, NewDesc->AssetData.ComponentMaskNum);
			for (int32 Index = 0; Index < MaxComponentMaskNum; ++Index)
			{
				ComponentMaskNames.Add(FName(BaseName, Index));
			}
		}
	}

	// Base material entry.
	FMaterialInstanceDiffResult BaseDiff;
	BaseDiff.InstancePath = InBaseMaterialPath;
	MaterialValidationLibrary::BuildBaseMaterialPropertyDiff(OldDesc->AssetData, NewDesc->AssetData, SwitchNames, ComponentMaskNames, BaseDiff.PropertyDiffs);
	if (BaseDiff.PropertyDiffs.Num())
	{
		BaseDiff.DiffType = EMaterialInstanceDiffType::Modified;
		OutDiffs.Add(MoveTemp(BaseDiff));
	}

	// Instance entries.
	const bool bLayoutsMatch = OldDesc->AssetData.StaticPropertyLayoutHash == NewDesc->AssetData.StaticPropertyLayoutHash;

	TSet<FString> AllInstancePaths;
	for (TPair<FString, int32> const& It : OldDesc->MaterialInstances) 
	{
		AllInstancePaths.Add(It.Key); 
	}
	for (TPair<FString, int32> const& It : NewDesc->MaterialInstances) 
	{
		AllInstancePaths.Add(It.Key); 
	}

	for (FString const& PathStr : AllInstancePaths)
	{
		FSoftObjectPath InstancePath = MaterialValidationLibrary::ConvertStringPathToSoftObjectPath(PathStr);

		int32 const* OldIndexPtr = OldDesc->MaterialInstances.Find(PathStr);
		if (OldIndexPtr == nullptr)
		{
			OutDiffs.Add({ .InstancePath = InstancePath, .DiffType = EMaterialInstanceDiffType::Added });
			continue;
		}
		
		int32 const* NewIndexPtr = NewDesc->MaterialInstances.Find(PathStr);
		if (NewIndexPtr == nullptr)
		{
			OutDiffs.Add({ .InstancePath = InstancePath, .DiffType = EMaterialInstanceDiffType::Removed });
			continue;
		}

		auto GetInstanceData = [](FMaterialValidationDesc const& Desc, int32 Index, int32 SwitchArrayNum, int32 MaskArrayNum) -> FMaterialValidationAssetData_MaterialInstance
		{
			if (Index == INDEX_NONE)
			{
				FMaterialValidationAssetData_MaterialInstance Default;
				Default.StaticSwitchOverrideMask.SetNumZeroed(SwitchArrayNum);
				Default.StaticSwitchOverrideValues.SetNumZeroed(SwitchArrayNum);
				Default.ComponentMaskOverrideMask.SetNumZeroed(MaskArrayNum);
				Default.ComponentMaskOverrideValues.SetNumZeroed(MaskArrayNum);
				return Default;
			}
			if (Desc.MaterialInstanceAssetDatas.IsValidIndex(Index))
			{
				return Desc.MaterialInstanceAssetDatas[Index];
			}
			return FMaterialValidationAssetData_MaterialInstance{};
		};

		const FMaterialValidationAssetData_MaterialInstance OldData = GetInstanceData(*OldDesc, *OldIndexPtr, OldDesc->AssetData.StaticSwitchValues.Num(), OldDesc->AssetData.ComponentMaskValues.Num());
		const FMaterialValidationAssetData_MaterialInstance NewData = GetInstanceData(*NewDesc, *NewIndexPtr, NewDesc->AssetData.StaticSwitchValues.Num(), NewDesc->AssetData.ComponentMaskValues.Num());

		FMaterialInstanceDiffResult InstanceDiff;
		InstanceDiff.InstancePath = InstancePath;
		MaterialValidationLibrary::BuildInstancePropertyDiff(OldData, NewData, SwitchNames, ComponentMaskNames, bLayoutsMatch, InstanceDiff.PropertyDiffs);

		if (InstanceDiff.PropertyDiffs.Num())
		{
			InstanceDiff.DiffType = EMaterialInstanceDiffType::Modified;
			OutDiffs.Add(MoveTemp(InstanceDiff));
		}
	}

	// Sort so that modified items appear before added/removed ones.
	// Note that if the base material is in the array it will in the first entry. And because it is guaranteed to be EMaterialInstanceDiffType::Modified the StableSort will keep it there.
	Algo::StableSort(OutDiffs, [](FMaterialInstanceDiffResult const& A, FMaterialInstanceDiffResult const& B)
	{
		return static_cast<uint8>(A.DiffType) < static_cast<uint8>(B.DiffType);
	});
}

#undef LOCTEXT_NAMESPACE
