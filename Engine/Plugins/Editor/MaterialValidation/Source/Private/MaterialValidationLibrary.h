// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "MaterialValidationLibrary.generated.h"

class UMaterial;
class UMaterialInterface;
class UMaterialInstanceConstant;
class UMaterialValidationGroup;

struct FMaterialDatabaseAssetHierarchyInfo;
struct FMaterialDatabaseAssetPropertyDesc;
struct FMaterialDatabaseAssetPropertyValue;
struct FMaterialInstanceDiffResult;

/** Material validation functionality exposed as a library of Blueprint functions. */
UCLASS(MinimalAPI, meta = (ScriptName = "MaterialValidationLibrary"))
class UMaterialValidationLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

	/** 
	 * Get all configured Material Validation Groups.
	 * 
	 * @param OutGroups		The returned array of groups.
	 * @param bInSyncLoad	Whether to sync load the group assets. When false we don't return unloaded group objects, but we do trigger async loading for them.
	 */
	UFUNCTION(BlueprintCallable, Category = Material)
	static void GetAllGroups(TArray<UMaterialValidationGroup*>& OutGroups, bool bInSyncLoad);

	/**
	 * Remove all UMaterials from a Material Validation Group.
	 * 
	 * @param InGroup	The group to reset.
	 */
	UFUNCTION(BlueprintCallable, Category = Material)
	static void ResetGroup(UMaterialValidationGroup* InGroup);

	/**
	 * Add all UMaterials to the Material Validation Group which are found in its material search directories.
	 * This will only add materials, and not remove existing ones.
	 * Materials are not updated with their permutations, so that newly found materials will have no approved permutations.
	 *
	 * @param InGroup	The group to add materials to.
	 */
	UFUNCTION(BlueprintCallable, Category = Material)
	static void AddMissingMaterialsToGroup(UMaterialValidationGroup* InGroup);

	/**
	 * Remove all UMaterials that no longer exist on disk from the Material Validation Group.
	 *
	 * @param InGroup	The group to update.
	 */
	UFUNCTION(BlueprintCallable, Category = Material)
	static void RemoveInvalidMaterialsFromGroup(UMaterialValidationGroup* InGroup);

	/**
	 * Recalculate and reregister all existing permutations for all the existing materials in a Material Validation Group (can be slow to load and analyze the tree of material instances).
	 *
	 * @param InGroup	The group to update.
	 */
	UFUNCTION(BlueprintCallable, Category = Material)
	static void UpdateMaterialPermutationsInGroup(UMaterialValidationGroup* InGroup);

	/**
	 * Resolve a asset path string to a FSoftObjectPath while handling asset redirectors.
	 *
	 * @param	InAssetPath		A asset path.
	 *
	 * @returns	The FSoftObjectPath of the resolved asset.
	 */
	static FSoftObjectPath ResolveAssetPath(FString const& InAssetPath);

	/**
	 * Get whether a material belongs to a group.
	 * 
	 * @param	InGroup				The group to use.
	 * @param	InMaterial			The material to validate.
	 * @param	bOutIsInGroupPath	Returns true if the material is in the validation group search path.
	 * @param	bOutIsInGroup		Returns true if the material is in the validation group material list.
	 */
	UFUNCTION(BlueprintCallable, Category = Material)
	static void IsMaterialInGroup(UMaterialValidationGroup const* InGroup, UMaterial* InMaterial, bool& bOutIsInGroupPath, bool& bOutIsInGroup);

	/**
	 * Get whether a material instance belongs to the group and whether it has a known permutation.
	 * 
	 * @param	InGroup							The group to use.
	 * @param	InMaterialInstance				The material instance to validate.
	 * @param	bOutMaterialInGroup				Returns true if the base material is in the validation group material list.
	 * @param	bOutMaterialPermutationInGroup	Returns true if the material instance permutation is in the permutation list.
	 */
	UFUNCTION(BlueprintCallable, Category = Material)
	static void IsMaterialInstanceInGroup(UMaterialValidationGroup const* InGroup, UMaterialInstanceConstant* InMaterialInstance, bool& bOutMaterialInGroup, bool& bOutMaterialPermutationInGroup);

	/**
	 * Get the number of shaders used by all permutations of a base material using only the information stored in the group.
	 *
	 * @param	InGroup					The group to use.
	 * @param	InMaterial				The base material to evaluate.
	 *
	 * @returns	The total number of shaders calculated at the last time we updated the group.
	 */
	UFUNCTION(BlueprintCallable, Category = Material)
	static int32 GetShaderCount(UMaterialValidationGroup const* InGroup, UMaterial* InMaterial);

	/**
	 * Get the number of shaders used by all permutations of a base material taking into account local changes in the materials.
	 *
	 * @param	InGroup					The group to use.
	 * @param	InMaterial				The base material to evaluate.
	 * @param	InModifiedObjects		An array of loaded material objects that should be used instead of the corresponding stored database values when evaluating permutations.
	 * @param	InReplacementObjects	An array of loaded material objects in sync with InModifiedObjects. When the corresponding array item exists and is valid it is used to replace 
	 *									the object in InModifiedObjects when evaluation permutations. This can be used to evaluate the impact of using a different object.
	 * @param	bForceLoadObjects		When true, any hierarchy asset not already in memory is synchronously loaded from disk.
	 *
	 * @returns	The total number of shaders. This value is the best estimate that we can give without triggering asset loads if bForceLoad=false.
	 */
	UFUNCTION(BlueprintCallable, Category = Material, meta = (AutoCreateRefTerm = "InReplacementObjects"))
	static int32 GetModifiedShaderCount(
		UMaterialValidationGroup const* InGroup, 
		UMaterial* InMaterial, 
		TArray<UMaterialInterface*> const& InModifiedObjects,
		TArray<UMaterialInterface*> const& InReplacementObjects,
		bool bForceLoadObjects = false);

	/**
	 * Get all of the material instance data for a single material hierarchy.
	 *
	 * @param	InGroup				The group to use.
	 * @param	InBaseMaterialPath	The path of the base material to evaluate.
	 * @param	OutInfo				Returnsa an object containing all of the material instance data.
	 */
	UFUNCTION(BlueprintCallable, Category = Material)
	static void GetMaterialHierarchyInfo(UMaterialValidationGroup const* InGroup, FSoftObjectPath const& InBaseMaterialPath, FMaterialDatabaseAssetHierarchyInfo& OutInfo);

	/**
	 * Get all the material properties for a material hierarchy.
	 *
	 * @param	InInfo			The full hierarchy info. This is typically resolved by first calling GetMaterialHierarchyInfo().
	 * @param	OutProperties	Returns an array of property descriptions.
	 */
	UFUNCTION(BlueprintCallable, Category = Material)
	static void GetMaterialProperties(FMaterialDatabaseAssetHierarchyInfo const& InInfo, TArray<FMaterialDatabaseAssetPropertyDesc>& OutProperties);
	
	/**
	 * Get all the material property values for a material instance in a material hierarchy.
	 *
	 * @param	InInfo			The full hierarchy info. This is typically resolved by first calling GetMaterialHierarchyInfo().
	 * @param	InMaterialIndex	The index of the material instance in the hierarchy. The instances are in the order defined by FMaterialDatabaseAssetHierarchyInfo::MaterialPaths.
	 * @param	OutValues		Returns an array of property values, one for each of the material properties as returned by GetProperties().
	 */
	UFUNCTION(BlueprintCallable, Category = Material)
	static void GetMaterialPropertyValues(FMaterialDatabaseAssetHierarchyInfo const& InInfo, int32 InMaterialIndex, TArray<FMaterialDatabaseAssetPropertyValue>& OutValues);

	/**
	 * Get all the currently edited material property values for a live, possibly unsaved, material instance object.
	 * The property layout (switch and mask names) is taken from InInfo and must be for the same base material.
	 *
	 * @param	InInfo				The full hierarchy info for the base material (from GetMaterialHierarchyInfo()).
	 * @param	InMaterialInstance	The material instance object to read values from.
	 * @param	OutValues			Returns an array of property values, one for each of the material properties as returned by GetProperties().
	 */
	static void GetMaterialPropertyValues(FMaterialDatabaseAssetHierarchyInfo const& InInfo, UMaterialInstanceConstant* InMaterialInstance, TArray<FMaterialDatabaseAssetPropertyValue>& OutValues);

	/**
	 * Get the diff between two versions of a UMaterialValidationGroup for a single base material.
	 * Compares the raw stored data. There is no asset registry dependency and it works for deleted instances.
	 * The first entry in OutDiffs always contains the base material entry if it changed.
	 *
	 * @param	InGroupOld				The older version of the group for comparison.
	 * @param	InGroupNew				The newer version of the group for comparison.
	 * @param	InBaseMaterialPath		The path of the base material to find in the groups and diff.
	 * @param	bAllowAssetLoad			If true then the base UMaterial is loaded to resolve static parameter names. If false then static parameters are given generic names.
	 * @param	OutDiffs				Returns one entry per changed material instance. Empty if no differences found.
	 */
	UFUNCTION(BlueprintCallable, Category = Material)
	static void GetMaterialValidationDescDiff(UMaterialValidationGroup const* InGroupOld, UMaterialValidationGroup const* InGroupNew, FSoftObjectPath const& InBaseMaterialPath, bool bAllowAssetLoad, TArray<FMaterialInstanceDiffResult>& OutDiffs);
};
