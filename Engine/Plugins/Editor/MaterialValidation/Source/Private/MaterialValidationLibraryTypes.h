// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MaterialValidationLibraryTypes.generated.h"

struct FMaterialValidationAssetData_MaterialInstance;

/** Enumeration of material instance property types. */
UENUM(BlueprintType)
enum class EMaterialPropertyCategory : uint8
{
	Default,
	StaticProperty,
	UsageFlag,
	StaticSwitch,
	ComponentMask,
	Count
};

/** Whether a material instance was added, removed, or modified between two revisions. */
UENUM(BlueprintType)
enum class EMaterialInstanceDiffType : uint8
{
	Modified,
	Added,
	Removed,
};

/** 
 * Structure that collects all the resolved asset data for one material hierarchy in the material asset database. 
 * Typically this is extracted from the compact form stored in a UMaterialValidationGroup and used in UI views or for further analysis.
 */
USTRUCT(BlueprintType)
struct FMaterialDatabaseAssetHierarchyInfo
{
	GENERATED_BODY()

	/** Array of material asset paths. This includes the base UMaterial asset in the first entry, and the child UMaterialInstance assets after that. */
	UPROPERTY(BlueprintReadOnly, Category = Material)
	TArray<FSoftObjectPath> MaterialPaths;
	/** Array of asset data info. This is synced with the MaterialPaths array. */
	UPROPERTY()
	TArray<FMaterialValidationAssetData_MaterialInstance> MaterialAssetDatas;
	/** Array of all StaticSwitch names. This matches the layout of all StaticSwitchOverrideValues arrays in the MaterialAssetDatas. */
	UPROPERTY()
	TArray<FName> StaticSwitchNames;
	/** Array of all ComponentMask names. This matches the layout of all ComponentMaskOverrideValues arrays in the MaterialAssetDatas. */
	UPROPERTY()
	TArray<FName> ComponentMaskNames;
};

/** Description of a material property in the material asset database. */
USTRUCT(BlueprintType)
struct FMaterialDatabaseAssetPropertyDesc
{
	GENERATED_BODY()

	/** Id for the property. */
	UPROPERTY(BlueprintReadOnly, Category = Material)
	FName Id;
	/** Readable name for the property. */
	UPROPERTY(BlueprintReadOnly, Category = Material)
	FName Name;
	/** Category for the property. */
	UPROPERTY(BlueprintReadOnly, Category = Material)
	EMaterialPropertyCategory Category = EMaterialPropertyCategory::Default;
};

/** Description of a resolved material value in the material asset database. */
USTRUCT(BlueprintType)
struct FMaterialDatabaseAssetPropertyValue
{
	GENERATED_BODY()

	/** String value for the property. */
	UPROPERTY(BlueprintReadOnly, Category = Material)
	FString Value;
	/** true if the value for the property is different from the value on its parent material. */
	UPROPERTY(BlueprintReadOnly, Category = Material)
	bool bIsModifiedFromParent = false;
};

/** A single property difference between two versions of a material instance. */
USTRUCT(BlueprintType)
struct FMaterialInstancePropertyDiff
{
	GENERATED_BODY()

	/** Human-readable name of the property. */
	UPROPERTY(BlueprintReadOnly, Category = Material)
	FName PropertyName;
	/** Category for the property. */
	UPROPERTY(BlueprintReadOnly, Category = Material)
	EMaterialPropertyCategory Category = EMaterialPropertyCategory::Default;
	/** String representation of the value in the old revision. "[Inherited]" if not overridden. */
	UPROPERTY(BlueprintReadOnly, Category = Material)
	FString OldValue;
	/** String representation of the value in the new revision. "[Inherited]" if not overridden. */
	UPROPERTY(BlueprintReadOnly, Category = Material)
	FString NewValue;
};

/** Diff result for a single material instance between two versions of a UMaterialValidationGroup. */
USTRUCT(BlueprintType)
struct FMaterialInstanceDiffResult
{
	GENERATED_BODY()

	/** Asset path of the material or material instance. */
	UPROPERTY(BlueprintReadOnly, Category = Material)
	FSoftObjectPath InstancePath;
	/** Whether this instance was added, removed, or modified. */
	UPROPERTY(BlueprintReadOnly, Category = Material)
	EMaterialInstanceDiffType DiffType = EMaterialInstanceDiffType::Modified;
	/** Per-property differences. This will be empty for Added and Removed entries. */
	UPROPERTY(BlueprintReadOnly, Category = Material)
	TArray<FMaterialInstancePropertyDiff> PropertyDiffs;
};
