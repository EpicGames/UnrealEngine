// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialValidationModule.h"

#include "Editor.h"
#include "EditorValidatorSubsystem.h"
#include "Materials/MaterialInstanceConstant.h"
#include "MaterialValidationAssetData.h"
#include "MaterialValidationConfig.h"
#include "MaterialValidationLibrary.h"
#include "MaterialValidationLibraryTypes.h"
#include "MaterialValidationSimilarityBrowser.h"
#include "MaterialValidators.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"

DEFINE_LOG_CATEGORY(LogMaterialValidation);

void FMaterialValidationModule::StartupModule()
{
	if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
	{
		FPropertyEditorModule& Module = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		Module.RegisterCustomClassLayout(
			UDataValidationSettings::StaticClass()->GetFName(),
			FOnGetDetailCustomizationInstance::CreateStatic(&FDataValidationSettingsCustomization::MakeInstance));
	}

	// We defer the parts of startup that depend on GEditor until after Editor Init.
	// Alternatively we could use the PostEngineInit LoadingPhase for this module.
	// But if we change the LoadingPhase we will need to use 2 modules for this plugin, because the Commandlet must be registered in LoadingPhase Default.
	FEditorDelegates::OnEditorInitialized.AddRaw(this, &FMaterialValidationModule::OnEditorInitialized);
	FEditorDelegates::OnEditorPreExit.AddRaw(this, &FMaterialValidationModule::OnEditorPreExit);
}

void FMaterialValidationModule::OnEditorInitialized(double Duration)
{
	if (GEditor != nullptr)
	{
		if (UMaterialEditorValidationSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMaterialEditorValidationSubsystem>())
		{
			UMaterialEditorValidatorBase* Validator = NewObject<UMaterialEditorValidator_Permutation>();
			Subsystem->AddValidator(Validator);
			MaterialEditorValidators.Add(Validator);
		}

		if (UEditorValidatorSubsystem* Subsystem = GEditor->GetEditorSubsystem<UEditorValidatorSubsystem>())
		{
			UEditorValidatorBase* Validator = NewObject<UEditorValidator_MaterialPermutation>();
			Subsystem->AddValidator(Validator);
			EditorValidators.Add(Validator);
		}
	}
}

void FMaterialValidationModule::OnEditorPreExit()
{
	if (GEditor != nullptr)
	{
		if (UMaterialEditorValidationSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMaterialEditorValidationSubsystem>())
		{
			for (TWeakObjectPtr<UMaterialEditorValidatorBase>& Validator : MaterialEditorValidators)
			{
				if (Validator.IsValid())
				{
					Subsystem->RemoveValidator(Validator.Get());
				}
			}
		}

		if (UEditorValidatorSubsystem* Subsystem = GEditor->GetEditorSubsystem<UEditorValidatorSubsystem>())
		{
			for (TWeakObjectPtr<UEditorValidatorBase>& Validator : EditorValidators)
			{
				if (Validator.IsValid())
				{
					Subsystem->RemoveValidator(Validator.Get());
				}
			}
		}
	}
}

void FMaterialValidationModule::ShutdownModule()
{
	if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
	{
		FPropertyEditorModule& Module = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		Module.UnregisterCustomClassLayout(UDataValidationSettings::StaticClass()->GetFName());
	}

	FEditorDelegates::OnEditorInitialized.RemoveAll(this);
	FEditorDelegates::OnEditorPreExit.RemoveAll(this);
}

TSharedPtr<SWindow> FMaterialValidationModule::OpenSimilarityBrowser(UMaterialInterface& MaterialInterface)
{
	UMaterial* BaseMaterial = MaterialInterface.GetMaterial();
	if (!BaseMaterial)
	{
		UE_LOGF(LogMaterialValidation, Error, "Unable to get Material from %ls", *MaterialInterface.GetName());
		return TSharedPtr<SWindow>();
	}

	// Resolve the owning group -- needed by the browser to populate the hierarchy list.
	UMaterialValidationGroup* TargetGroup = nullptr;
	{
		TArray<UMaterialValidationGroup*> AllGroups;
		UMaterialValidationLibrary::GetAllGroups(AllGroups, /*bSyncLoad=*/true);

		for (UMaterialValidationGroup* Group : AllGroups)
		{
			bool bInGroupPath = false, bInGroup = false;
			UMaterialValidationLibrary::IsMaterialInGroup(Group, BaseMaterial, bInGroupPath, bInGroup);
			if (bInGroup)
			{
				TargetGroup = Group;
				break;
			}
		}
	}

	if (!TargetGroup)
	{
		UE_LOGF(LogMaterialValidation, Error, "Unable to find a validation group for material %ls", *MaterialInterface.GetName());
		return TSharedPtr<SWindow>();
	}

	return SMaterialInstanceSimilarityBrowser::CreateWindow(TargetGroup, FSoftObjectPath(BaseMaterial), Cast<UMaterialInstanceConstant>(&MaterialInterface));
}

IMPLEMENT_MODULE(FMaterialValidationModule, MaterialValidation)
