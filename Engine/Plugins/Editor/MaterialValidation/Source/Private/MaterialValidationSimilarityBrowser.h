// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MaterialValidationLibraryTypes.h"
#include "Widgets/SCompoundWidget.h"
#include "MaterialValidationSimilarityBrowser.generated.h"

class UMaterial;
class UMaterialInstanceConstant;
class UMaterialValidationGroup;
namespace MaterialValidation { class SSimilarityBrowserWidget; }

/** Transient settings object for the similarity browser, exposed to details panel UI. */
UCLASS(Transient)
class USimilarityBrowserSettings : public UObject
{
	GENERATED_BODY()

public:
	/** Base material whose hierarchy is searched for similar instances. */
	UPROPERTY(EditAnywhere, Category="Selection", meta=(NoResetToDefault))
	TSoftObjectPtr<UMaterial> BaseMaterial;
	/** The current material instance being compared against the hierarchy. */
	UPROPERTY(EditAnywhere, Category="Selection", meta=(NoResetToDefault))
	TObjectPtr<UMaterialInstanceConstant> CurrentInstance;

	/** When true, shows only one row for each unique material permutation and also hides rows which have the same permutation as the current instance. */
	UPROPERTY(EditAnywhere, Category="Filter")
	bool bShowUniqueOnly = true;
	UPROPERTY(EditAnywhere, Category="Filter", meta=(InlineEditConditionToggle))
	bool bUseMaxDistance = false;
	/** Hides similarity rows whose distance exceeds this value. */
	UPROPERTY(EditAnywhere, Category="Filter", meta=(ClampMin=0, EditCondition="bUseMaxDistance"))
	float MaxDistance = 0.f;

	/** Distance contribution per different static property (BlendMode, TwoSided, etc.). */
	UPROPERTY(EditAnywhere, Category="Category Weights", meta=(ClampMin=0, ClampMax=10))
	float StaticPropertyWeight = 1.f;
	/** Distance contribution per different usage flag. */
	UPROPERTY(EditAnywhere, Category="Category Weights", meta=(ClampMin=0, ClampMax=10))
	float UsageFlagWeight = 1.f;
	/** Distance contribution per different static switch value. */
	UPROPERTY(EditAnywhere, Category="Category Weights", meta=(ClampMin=0, ClampMax=10))
	float StaticSwitchWeight = 1.f;
	/** Distance contribution per different component mask value. */
	UPROPERTY(EditAnywhere, Category="Category Weights", meta=(ClampMin=0, ClampMax=10))
	float ComponentMaskWeight = 1.f;

	/** Returns the distance weight for the given property category. */
	float GetWeightForCategory(EMaterialPropertyCategory Cat) const;
};

/**
 * Widget that finds existing material instances in a hierarchy that are "close" (fewest different properties) to a live material instance being edited. 
 * Intended to help a user discover approved permutations when validation warns that their instance creates a new one.
 */
class SMaterialInstanceSimilarityBrowser : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMaterialInstanceSimilarityBrowser) {}
		SLATE_ARGUMENT(UMaterialValidationGroup const*, Group)
		SLATE_ARGUMENT(FSoftObjectPath, BaseMaterialPath)
		SLATE_ARGUMENT(UMaterialInstanceConstant*, CurrentInstance)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	/** Create and display a standalone similarity browser window. */
	static TSharedPtr<SWindow> CreateWindow(UMaterialValidationGroup const* InGroup, FSoftObjectPath const& InBaseMaterialPath, UMaterialInstanceConstant* InCurrentInstance);

	/** Update an existing browser to compare a different instance. */
	void NavigateTo(UMaterialValidationGroup const* InGroup, FSoftObjectPath const& InBaseMaterialPath, UMaterialInstanceConstant* InCurrentInstance);

private:
	/** Inner widget that owns all state and layout for the browser. */
	TSharedPtr<class MaterialValidation::SSimilarityBrowserWidget> BrowserWidget;
};
