// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialValidationToolkit.h"

#include "AssetRegistry/IAssetRegistry.h"
#include "Framework/Application/SlateApplication.h"
#include "Hash/xxhash.h"
#include "Materials/MaterialInterface.h"
#include "MaterialValidationChangelistTab.h"
#include "MaterialValidationGroup.h"
#include "MaterialValidationGroupDiff.h"
#include "MaterialValidationLibrary.h"
#include "MaterialValidationLibraryTypes.h"
#include "MaterialValidationToolkitShared.h"
#include "Misc/NotifyHook.h"
#include "Misc/TextFilterExpressionEvaluator.h"
#include "Modules/ModuleManager.h"
#include "PropertyCustomizationHelpers.h"  
#include "PropertyEditorModule.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "UObject/GCObject.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "MaterialValidation"

namespace MaterialValidation {

float GAssetNameColumnWidth = 700.f;

const FText GEmptyText = FText::FromString("-");
const FText GUnknownText = FText::FromString("?");

FSlateColor GetDeltaColor(int32 InDelta)
{
	if (InDelta > 0)
	{
		return FSlateColor(FLinearColor(1.f, 0.4f, 0.4f));
	}
	else if (InDelta < 0)
	{
		return FSlateColor(FLinearColor(0.4f, 1.f, 0.4f));
	}
	return FSlateColor::UseForeground();
}

FText GetTextSignedInt(int32 Value)
{
	return (Value > 0) ? FText::Format(LOCTEXT("PositiveNumberFormat", "+{0}"), FText::AsNumber(Value)) : FText::AsNumber(Value);
}

TSharedRef<SWidget> GenerateAssetPathWidget(FSoftObjectPath InAssetPath, FSlateColor InColor)
{
	const FAssetData AssetData = IAssetRegistry::Get()->GetAssetByObjectPath(InAssetPath);
	const EVisibility BrowseVisibility = AssetData.IsValid() ? EVisibility::Visible : EVisibility::Hidden;

	return SNew(SBox)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.Text(FText::FromString(InAssetPath.ToString()))
						.ToolTipText(FText::FromString(InAssetPath.ToString()))
						.ColorAndOpacity(InColor)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4.0f, 0.0f)
				[
					SNew(SBox)
						.Visibility(BrowseVisibility)
						[
							PropertyCustomizationHelpers::MakeBrowseButton(
								FSimpleDelegate::CreateLambda([AssetData]()
									{
										const FModifierKeysState KeyState = FSlateApplication::Get().GetModifierKeys();
										if (KeyState.IsAltDown())
										{
											GEditor->EditObject(AssetData.GetAsset());
										}
										else
										{
											GEditor->SyncBrowserToObjects({ AssetData });
										}
									}),
								LOCTEXT("BrowseTooltip", "Browse to Asset in Content Browser"))
						]
				]
		];
}

/** A data table entry describing a UMaterial. */
struct FMaterialStats
{
	FSoftObjectPath AssetPath;
	int32 MaterialInstanceCount = 0;
	int32 StaticPermutationCount = 0;
	int32 ShaderCount = 0;
	int32 StaticSwitchCount = 0;
	int32 ComponentMaskCount = 0;

	// Stat table columns are referenced by FName.
	static const FName ColumnId_AssetPath;
	static const FName ColumnId_MaterialInstanceCount;
	static const FName ColumnId_StaticPermutationCount;
	static const FName ColumnId_ShaderCount;
	static const FName ColumnId_StaticSwitchCount;
	static const FName ColumnId_ComponentMaskCount;

	FMaterialStats(FSoftObjectPath const& InPath, int32 InInstanceCount, int32 InPermutationCount, int32 InShaderCount, int32 InStaticSwitchCount, int32 InComponentMaskCount)
		: AssetPath(InPath)
		, MaterialInstanceCount(InInstanceCount)
		, StaticPermutationCount(InPermutationCount)
		, ShaderCount(InShaderCount)
		, StaticSwitchCount(InStaticSwitchCount)
		, ComponentMaskCount(InComponentMaskCount)
	{
	}
};

const FName FMaterialStats::ColumnId_AssetPath("CID_AssetPath");
const FName FMaterialStats::ColumnId_MaterialInstanceCount("CID_MaterialInstanceCount");
const FName FMaterialStats::ColumnId_StaticPermutationCount("CID_StaticPermutationCount");
const FName FMaterialStats::ColumnId_ShaderCount("CID_ShaderCount");
const FName FMaterialStats::ColumnId_StaticSwitchCount("CID_StaticSwitchCount");
const FName FMaterialStats::ColumnId_ComponentMaskCount("CID_ComponentMaskCount");

struct FExperimentData
{
	FString Name;
	uint32 ReductionCount = 0;
	float ReductionPercent = 0.0f;

	static const FName ColumnId_Name;
	static const FName ColumnId_ReductionCount;
	static const FName ColumnId_ReductionPercent;
};

const FName FExperimentData::ColumnId_Name("CID_ExperimentName");
const FName FExperimentData::ColumnId_ReductionCount("CID_ReductionCount");
const FName FExperimentData::ColumnId_ReductionPercent("CID_ReductionPercent");

class SExperimentTableRow : public SMultiColumnTableRow<TSharedPtr<FExperimentData>>
{
	TSharedPtr<FExperimentData> RowItem;

public:
	SLATE_BEGIN_ARGS(SExperimentTableRow) {}
		SLATE_ARGUMENT(TSharedPtr<FExperimentData>, Item)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable)
	{
		RowItem = InArgs._Item;
		SMultiColumnTableRow<TSharedPtr<FExperimentData>>::Construct(FSuperRowType::FArguments(), InOwnerTable);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (RowItem.IsValid())
		{
			if (ColumnName == FExperimentData::ColumnId_Name)
			{
				return SNew(STextBlock).Text(FText::FromString(RowItem->Name));
			}
			if (ColumnName == FExperimentData::ColumnId_ReductionCount)
			{
				return SNew(STextBlock).Text(FText::AsNumber(RowItem->ReductionCount));
			}
			if (ColumnName == FExperimentData::ColumnId_ReductionPercent)
			{
				return SNew(STextBlock).Text(FText::AsPercent(RowItem->ReductionPercent));
			}
		}
		return SNew(STextBlock).Text(FText::FromString("?"));
	}
};

class SExperimentsTable : public SCompoundWidget
{
	TSharedPtr<SListView<TSharedPtr<FExperimentData>>> ListView;
	TSharedPtr<SHeaderRow> HeaderRow;
	TArray<TSharedPtr<FExperimentData>> Rows;
	float NameColumnWidth = 500.0f;

public:
	SLATE_BEGIN_ARGS(SExperimentsTable) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		HeaderRow = SNew(SHeaderRow)
			+ SHeaderRow::Column(FExperimentData::ColumnId_Name)
				.HAlignHeader(HAlign_Center).HAlignCell(HAlign_Left)
				.DefaultLabel(LOCTEXT("ExperimentName", "Experiment"))
				.ManualWidth(NameColumnWidth)
				.ManualWidth_Lambda([this]() { return NameColumnWidth; })
				.OnWidthChanged_Lambda([this](float NewWidth) { NameColumnWidth = NewWidth; })
			+ SHeaderRow::Column(FExperimentData::ColumnId_ReductionCount)
				.HAlignHeader(HAlign_Center).HAlignCell(HAlign_Center)
				.DefaultLabel(LOCTEXT("ExperimentReductionCount", "Reduction"))
				.FixedWidth(120)
			+ SHeaderRow::Column(FExperimentData::ColumnId_ReductionPercent)
				.HAlignHeader(HAlign_Center).HAlignCell(HAlign_Center)
				.DefaultLabel(LOCTEXT("ExperimentReductionPercent", "Reduction %"))
				.FixedWidth(120);

		ChildSlot
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot().HAlign(HAlign_Fill).FillHeight(1.0f)
			[
				SNew(SScrollBox)
				.Orientation(Orient_Horizontal)
				+SScrollBox::Slot()
				[
					SAssignNew(ListView, SListView<TSharedPtr<FExperimentData>>)
					.HeaderRow(HeaderRow.ToSharedRef())
					.ListItemsSource(&Rows)
					.OnGenerateRow_Lambda([this](TSharedPtr<FExperimentData> InItem, const TSharedRef<STableViewBase>& InOwnerTable)
						{
							return SNew(SExperimentTableRow, InOwnerTable).Item(InItem);
						})
				]
			]
		];
	}

	/** Set the table content. */
	void SetContent(TArray<TSharedPtr<FExperimentData>> const& InRows)
	{
		Rows = InRows;
		ListView->RequestListRefresh();
	}
};

static TSharedPtr<SExperimentsTable> CreateExperimentsTable()
{
	return SNew(SExperimentsTable);
}

/** A slate container of a data table entry describing a UMaterial. */
class SMaterialTableRow : public SMultiColumnTableRow<TSharedPtr<FMaterialStats>>
{
	/** The material stats for the row. */
	TSharedPtr<FMaterialStats> RowItem;

public:
	SLATE_BEGIN_ARGS(SMaterialTableRow) {}
		SLATE_ARGUMENT(TSharedPtr<FMaterialStats>, Item)
	SLATE_END_ARGS()

	/** Builds the row. */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable)
	{
		RowItem = InArgs._Item;
		SMultiColumnTableRow<TSharedPtr<FMaterialStats>>::Construct(FSuperRowType::FArguments(), InOwnerTable);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (RowItem.IsValid())
		{
			if (ColumnName == FMaterialStats::ColumnId_AssetPath)
			{
				return GenerateAssetPathWidget(RowItem->AssetPath, FSlateColor::UseForeground());
			}
			if (ColumnName == FMaterialStats::ColumnId_MaterialInstanceCount)
			{
				return SNew(STextBlock).Text(FText::AsNumber(RowItem->MaterialInstanceCount));
			}
			if (ColumnName == FMaterialStats::ColumnId_StaticPermutationCount)
			{
				return SNew(STextBlock).Text(FText::AsNumber(RowItem->StaticPermutationCount));
			}
			if (ColumnName == FMaterialStats::ColumnId_ShaderCount)
			{
				return SNew(STextBlock).Text(FText::AsNumber(RowItem->ShaderCount));
			}
			if (ColumnName == FMaterialStats::ColumnId_StaticSwitchCount)
			{
				return SNew(STextBlock).Text(FText::AsNumber(RowItem->StaticSwitchCount));
			}
			if (ColumnName == FMaterialStats::ColumnId_ComponentMaskCount)
			{
				return SNew(STextBlock).Text(FText::AsNumber(RowItem->ComponentMaskCount));
			}
		}
		return SNew(STextBlock).Text(FText::FromString("?"));
	}
};

/** Delegate for notification of material data row selection. */
DECLARE_DELEGATE_OneParam(FOnRowItemSelect, TSharedPtr<FMaterialStats>);

/** A slate widget containing a table of UMaterial descriptions. */
class SMaterialTable : public SCompoundWidget
{
	/** All the available rows. */
	TArray<TSharedPtr<FMaterialStats>> AllRows;
	/** An ordered and filtered subset of AllRows. */
	TArray<TSharedPtr<FMaterialStats>> FilteredRows;
	/** A slate view of the table. */
	TSharedPtr<SListView<TSharedPtr<FMaterialStats>>> ListView;

	/** Search text used for filtering the rows. */
	FString SearchText;
	/** Selected column name for sorting the table. */
	FName SortColumnId = NAME_None;
	/** Selected mode for sorting the table. */
	EColumnSortMode::Type SortMode = EColumnSortMode::None;

	/** Delegate for row item selection. */
	FOnRowItemSelect OnRowItemSelect;

public:
	SLATE_BEGIN_ARGS(SMaterialTable) {}
		SLATE_ARGUMENT(TArray<TSharedPtr<FMaterialStats>>, InitialRows)
		SLATE_EVENT(FOnRowItemSelect, OnRowItemSelect)
	SLATE_END_ARGS()

	/** Builds the slate table view. */
	void Construct(const FArguments& InArgs)
	{
		AllRows = InArgs._InitialRows;
		FilteredRows = InArgs._InitialRows;
		OnRowItemSelect = InArgs._OnRowItemSelect;

		SearchText.Reset();
		
		SortColumnId = FMaterialStats::ColumnId_StaticPermutationCount;
		SortMode = EColumnSortMode::Descending;
		UpdateSort();

		ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot().AutoHeight().Padding(2)
			[
				SNew(SSearchBox)
				.HintText(LOCTEXT("MaterialSearchHint", "Filter By Asset"))
				.OnTextChanged(this, &SMaterialTable::OnSearchTextChanged)
			]
			+ SVerticalBox::Slot().FillHeight(1.0f).Padding(2)
			.FillHeight(1.f)
			[
				SAssignNew(ListView, SListView<TSharedPtr<FMaterialStats>>)
				.HeaderRow
				(
					SNew(SHeaderRow)
					+ SHeaderRow::Column(FMaterialStats::ColumnId_AssetPath)
						.HAlignHeader(HAlign_Center)
						.HAlignCell(HAlign_Left)
						.DefaultLabel(LOCTEXT("MaterialAssetPath", "Asset Path"))
						.OnSort(this, &SMaterialTable::OnColumnSortModeChanged)
						.ManualWidth(GAssetNameColumnWidth)
						.ManualWidth_Lambda([]() { return GAssetNameColumnWidth; })
						.OnWidthChanged_Lambda([](float NewWidth) { GAssetNameColumnWidth = NewWidth; })
					+ SHeaderRow::Column(FMaterialStats::ColumnId_MaterialInstanceCount)
						.HAlignHeader(HAlign_Center)
						.HAlignCell(HAlign_Center)
						.DefaultLabel(LOCTEXT("MaterialInstanceCount", "Instance Count"))
						.OnSort(this, &SMaterialTable::OnColumnSortModeChanged)
						.FixedWidth(150)
					+ SHeaderRow::Column(FMaterialStats::ColumnId_StaticPermutationCount)
						.HAlignHeader(HAlign_Center)
						.HAlignCell(HAlign_Center)
						.DefaultLabel(LOCTEXT("MaterialPermutationCount", "Permutation Count"))
						.OnSort(this, &SMaterialTable::OnColumnSortModeChanged)
						.FixedWidth(150)
					+ SHeaderRow::Column(FMaterialStats::ColumnId_ShaderCount)
						.HAlignHeader(HAlign_Center)
						.HAlignCell(HAlign_Center)
						.DefaultLabel(LOCTEXT("MaterialShaderCount", "Shader Count"))
						.OnSort(this, &SMaterialTable::OnColumnSortModeChanged)
						.FixedWidth(150)
					+ SHeaderRow::Column(FMaterialStats::ColumnId_StaticSwitchCount)
						.HAlignHeader(HAlign_Center)
						.HAlignCell(HAlign_Center)
						.DefaultLabel(LOCTEXT("MaterialStaticSwitchCount", "Static Switch Count"))
						.OnSort(this, &SMaterialTable::OnColumnSortModeChanged)
						.FixedWidth(150)
					+ SHeaderRow::Column(FMaterialStats::ColumnId_ComponentMaskCount)
						.HAlignHeader(HAlign_Center)
						.HAlignCell(HAlign_Center)
						.DefaultLabel(LOCTEXT("MaterialComponentMaskCount", "Component Mask Count"))
						.OnSort(this, &SMaterialTable::OnColumnSortModeChanged)
						.FixedWidth(150)
				)
				.ListItemsSource(&FilteredRows)
				.OnGenerateRow_Lambda([](TSharedPtr<FMaterialStats> InItem, const TSharedRef<STableViewBase>& InOwnerTable)
				{
					return SNew(SMaterialTableRow, InOwnerTable).Item(InItem);
				})
				.SelectionMode(ESelectionMode::Single)
				.OnMouseButtonDoubleClick(OnRowItemSelect)
			]
		];
	}

	/** Called when modifying the search text. */
	void OnSearchTextChanged(const FText& InText)
	{
		SearchText = InText.ToString();
		
		UpdateFilter();
		UpdateSort();
		UpdateListView();
	}

	/** Called when clicking on a column header to sort. */
	void OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode)
	{
		if (SortColumnId != ColumnId)
		{
			// Initial sort mode is ascending for asset path, descending for numeric.
			SortMode = ColumnId == FMaterialStats::ColumnId_AssetPath ? EColumnSortMode::Ascending : EColumnSortMode::Descending;
			SortColumnId = ColumnId;
		}
		else
		{
			SortMode = SortMode == EColumnSortMode::Descending ? EColumnSortMode::Ascending : EColumnSortMode::Descending;
		}

		UpdateSort();
		UpdateListView();
	}

	/**  Apply filter to AllRows to recreate FilteredRows. */
	void UpdateFilter()
	{
		FilteredRows.Reset();

		FTextFilterExpressionEvaluator Filter(ETextFilterExpressionEvaluatorMode::BasicString);
		Filter.SetFilterText(FText::FromString(SearchText));

		for (const TSharedPtr<FMaterialStats>& Row : AllRows)
		{
			if (SearchText.IsEmpty() || Filter.TestTextFilter(FBasicStringFilterExpressionContext(Row->AssetPath.ToString())))
			{
				FilteredRows.Add(Row);
			}
		}
	}

	/**  Apply sort to FilteredRows. */
	void UpdateSort()
	{
		const bool bSortModifier = SortMode == EColumnSortMode::Descending;

		if (SortColumnId == FMaterialStats::ColumnId_AssetPath)
		{
			FilteredRows.Sort([bSortModifier](TSharedPtr<FMaterialStats> const& A, TSharedPtr<FMaterialStats> const& B)
			{
				return bSortModifier ? A->AssetPath.LexicalLess(B->AssetPath) : B->AssetPath.LexicalLess(A->AssetPath);
			});
		}
		else if (SortColumnId == FMaterialStats::ColumnId_MaterialInstanceCount)
		{
			FilteredRows.Sort([bSortModifier](TSharedPtr<FMaterialStats> const& A, TSharedPtr<FMaterialStats> const& B)
			{
				return bSortModifier ? (A->MaterialInstanceCount > B->MaterialInstanceCount) : (A->MaterialInstanceCount < B->MaterialInstanceCount);
			});
		}
		else if (SortColumnId == FMaterialStats::ColumnId_StaticPermutationCount)
		{
			FilteredRows.Sort([bSortModifier](TSharedPtr<FMaterialStats> const& A, TSharedPtr<FMaterialStats> const& B)
			{
				return bSortModifier ? (A->StaticPermutationCount > B->StaticPermutationCount) : (A->StaticPermutationCount < B->StaticPermutationCount);
			});
		}
		else if (SortColumnId == FMaterialStats::ColumnId_ShaderCount)
		{
			FilteredRows.Sort([bSortModifier](TSharedPtr<FMaterialStats> const& A, TSharedPtr<FMaterialStats> const& B)
			{
				return bSortModifier ? (A->ShaderCount > B->ShaderCount) : (A->ShaderCount < B->ShaderCount);
			});
		}
		else if (SortColumnId == FMaterialStats::ColumnId_StaticSwitchCount)
		{
			FilteredRows.Sort([bSortModifier](TSharedPtr<FMaterialStats> const& A, TSharedPtr<FMaterialStats> const& B)
			{
				return bSortModifier ? (A->StaticSwitchCount > B->StaticSwitchCount) : (A->StaticSwitchCount < B->StaticSwitchCount);
			});
		}
		else if (SortColumnId == FMaterialStats::ColumnId_ComponentMaskCount)
		{
			FilteredRows.Sort([bSortModifier](TSharedPtr<FMaterialStats> const& A, TSharedPtr<FMaterialStats> const& B)
			{
				return bSortModifier ? (A->ComponentMaskCount > B->ComponentMaskCount) : (A->ComponentMaskCount < B->ComponentMaskCount);
			});
		}
	}

	/** Update the view. Must be called after any call to UpdateFilter() or UpdateSort(). */
	void UpdateListView()
	{
		if (ListView.IsValid())
		{
			ListView->RequestListRefresh();
		}
	}
};

/** Instantiate a table widget for all materials in a group. */
static TSharedPtr<SMaterialTable> CreateMaterialTable(UMaterialValidationGroup const* InGroup, FOnRowItemSelect OnRowItemSelect)
{
	TArray<TSharedPtr<FMaterialStats>> Stats;
	if (InGroup != nullptr)
	{
		for (TPair<FString, FMaterialValidationDesc> const& It : InGroup->Materials)
		{
			Stats.Add(MakeShared<FMaterialStats>(
				UMaterialValidationLibrary::ResolveAssetPath(It.Key),
				It.Value.MaterialInstances.Num(), 
				It.Value.PermutationHashes.Num() + 1,	// PermutationHashes doesn't include hash for base material 
				It.Value.NumShadersTotal, 
				It.Value.AssetData.StaticSwitchNum,
				It.Value.AssetData.ComponentMaskNum));
		}
	}
	return SNew(SMaterialTable).InitialRows(Stats).OnRowItemSelect(OnRowItemSelect);
}

/** The static switch and static component mask parameter FNames used by a single material. */
struct FMaterialInstanceParameterNames
{
	TArray<FName> StaticSwitches;
	TMap<FName, int32> StaticSwitchNameToIndex;
	TArray<FName> ComponentMasks;
	TMap<FName, int32> ComponentMaskNameToIndex;
};

/** A description for a single column in the UMaterialInstance data table. */
struct FMaterialInstanceColumnData
{
	/** FName Id used for column resolution. */
	FName Id;
	/** FText label used for display name. */
	FText Label;
	/** Column category used for filtering. */
	EMaterialPropertyCategory Category;

	FMaterialInstanceColumnData(FName const& InId, FText const& InLabel, EMaterialPropertyCategory InCategory)
		: Id(InId)
		, Label(InLabel)
		, Category(InCategory)
	{
	}
};

static uint32 GetNoUsageHash(const FStaticPermutationProperties& Properties)
{
	FStaticPermutationProperties NoUsageProperties = Properties;
	NoUsageProperties.UsageFlags = 0;
	return GetTypeHash(NoUsageProperties);
}

static uint32 GetArrayHash(const TArray<uint32>& Array)
{
	FXxHash64 Hash = FXxHash64::HashBuffer(Array);
	return (uint32)(Hash.Hash);
}

/** A data table entry describing a UMaterialInstance. */
struct FMaterialInstanceRowData
{
	/** Path to the UMaterialInstance asset. */
	FSoftObjectPath AssetPath;
	/** Index in the material instance hierarchy used when sorting by name. */
	int32 HierarchyIndex = 0;
	/** Full resolved asset data description for the UMaterialInstance pulled from the material validation database. */
	FMaterialValidationAssetData_MaterialInstance AssetData;

	/** Per-category hash data, derived from instance data */
	uint32 PerCategoryHash[(int32)EMaterialPropertyCategory::Count] = {};

	FMaterialInstanceRowData(FSoftObjectPath const& InPath, int32 InHierarchyIndex, FMaterialValidationAssetData_MaterialInstance const& InAssetData)
		: AssetPath(InPath)
		, HierarchyIndex(InHierarchyIndex)
		, AssetData(InAssetData)
	{
		PerCategoryHash[(int32)EMaterialPropertyCategory::StaticProperty] = GetNoUsageHash(InAssetData.StaticProperties);
		PerCategoryHash[(int32)EMaterialPropertyCategory::UsageFlag] = InAssetData.StaticProperties.UsageFlags;
		PerCategoryHash[(int32)EMaterialPropertyCategory::StaticSwitch] = GetArrayHash(InAssetData.StaticSwitchOverrideValues);
		PerCategoryHash[(int32)EMaterialPropertyCategory::ComponentMask] = GetArrayHash(InAssetData.ComponentMaskOverrideValues);
	}
};

/** 
 * Enumeration of all of the UMaterialInstance base properties. 
 * Aside from AssetPath and PermutationHash (kept in same enum for simplicity) these are the property overrides that trigger static permutations.
 */
enum class EMaterialInstanceStaticProperties : int32
{
	AssetPath,
	PermutationHash,
	TwoSided,
	bIsThinSurface,
	DitheredLODTransition,
	bCastDynamicShadowAsMasked,
	bOutputTranslucentVelocity,
	bHasPixelAnimation,
	bEnableTessellation,
	BlendMode,
	ShadingModel,
	OpacityMaskClipValue,
	Count
};

/** Helper getter functions for each of the EMaterialInstanceStaticProperties. */
struct FMaterialInstanceStaticPropertyGetters
{
	static FSoftObjectPath const& GetAssetPath(FMaterialInstanceRowData const& P) { return P.AssetPath; }
	static int32 GetAssetPathIndex(FMaterialInstanceRowData const& P) { return P.HierarchyIndex; }
	static uint32 GetPermutationHash(FMaterialInstanceRowData const& P) { return P.AssetData.PermutationHash; }
	static uint8 GetTwoSided(FMaterialInstanceRowData const& P) { return P.AssetData.StaticProperties.TwoSided; }
	static uint8 GetIsThinSurface(FMaterialInstanceRowData const& P) { return P.AssetData.StaticProperties.bIsThinSurface; }
	static uint8 GetDitheredLODTransition(FMaterialInstanceRowData const& P) { return P.AssetData.StaticProperties.DitheredLODTransition; }
	static uint8 GetCastDynamicShadowAsMasked(FMaterialInstanceRowData const& P) { return P.AssetData.StaticProperties.bCastDynamicShadowAsMasked; }
	static uint8 GetOutputTranslucentVelocity(FMaterialInstanceRowData const& P) { return P.AssetData.StaticProperties.bOutputTranslucentVelocity; }
	static uint8 GetHasPixelAnimation(FMaterialInstanceRowData const& P) { return P.AssetData.StaticProperties.bHasPixelAnimation; }
	static uint8 GetEnableTessellation(FMaterialInstanceRowData const& P) { return P.AssetData.StaticProperties.bEnableTessellation; }
	static TEnumAsByte<EBlendMode> GetBlendMode(FMaterialInstanceRowData const& P) { return P.AssetData.StaticProperties.BlendMode; }
	static TEnumAsByte<EMaterialShadingModel> GetShadingModel(FMaterialInstanceRowData const& P) { return P.AssetData.StaticProperties.ShadingModel; }
	static float GetOpacityMaskClipValue(FMaterialInstanceRowData const& P) { return P.AssetData.StaticProperties.OpacityMaskClipValue; }
};

struct FMaterialInstanceStaticPropertyModifiedGetters
{
	static bool GetAssetPathModified(FMaterialInstanceRowData const& P) { return false; }
	static bool GetPermutationHashModified(FMaterialInstanceRowData const& P) { return false; }
	static bool GetTwoSidedModified(FMaterialInstanceRowData const& P) { return P.AssetData.StaticPropertyOverrideFlags.bOverride_TwoSided; }
	static bool GetIsThinSurfaceModified(FMaterialInstanceRowData const& P) { return P.AssetData.StaticPropertyOverrideFlags.bOverride_bIsThinSurface; }
	static bool GetDitheredLODTransitionModified(FMaterialInstanceRowData const& P) { return P.AssetData.StaticPropertyOverrideFlags.bOverride_DitheredLODTransition; }
	static bool GetCastDynamicShadowAsMaskedModified(FMaterialInstanceRowData const& P) { return P.AssetData.StaticPropertyOverrideFlags.bOverride_CastDynamicShadowAsMasked; }
	static bool GetOutputTranslucentVelocityModified(FMaterialInstanceRowData const& P) { return P.AssetData.StaticPropertyOverrideFlags.bOverride_OutputTranslucentVelocity; }
	static bool GetHasPixelAnimationModified(FMaterialInstanceRowData const& P) { return P.AssetData.StaticPropertyOverrideFlags.bOverride_bHasPixelAnimation; }
	static bool GetEnableTessellationModified(FMaterialInstanceRowData const& P) { return P.AssetData.StaticPropertyOverrideFlags.bOverride_bEnableTessellation; }
	static bool GetBlendModeModified(FMaterialInstanceRowData const& P) { return P.AssetData.StaticPropertyOverrideFlags.bOverride_BlendMode; }
	static bool GetShadingModelModified(FMaterialInstanceRowData const& P) { return P.AssetData.StaticPropertyOverrideFlags.bOverride_ShadingModel; }
	static bool GetOpacityMaskClipModified(FMaterialInstanceRowData const& P) { return P.AssetData.StaticPropertyOverrideFlags.bOverride_OpacityMaskClipValue; }
};

/** Templatized sort functor for each of the EMaterialInstanceStaticProperties. */
template <auto GetterPtr>
static void SortByStaticProperty(TArray<TSharedPtr<FMaterialInstanceRowData>>& InArray, bool bInSortModifer)
{
	if (bInSortModifer)
	{
		Algo::StableSort(InArray, [](TSharedPtr<FMaterialInstanceRowData> const& A, TSharedPtr<FMaterialInstanceRowData> const& B) { return (*GetterPtr)(*A) > (*GetterPtr)(*B); });
	}
	else
	{
		Algo::StableSort(InArray, [](TSharedPtr<FMaterialInstanceRowData> const& A, TSharedPtr<FMaterialInstanceRowData> const& B) { return (*GetterPtr)(*A) < (*GetterPtr)(*B); });
	}
}

using PropertySortFunction = void(*)(TArray<TSharedPtr<FMaterialInstanceRowData>>&, bool);

static FSlateColor GetColor(bool bModified)
{
	return bModified ? FSlateColor(EStyleColor::AccentYellow) : FSlateColor(EStyleColor::Foreground);
}

/** Templatized table entry widget generator for each of the EMaterialInstanceStaticProperties. */
template <auto GetterPtr, auto ModifiedGetterPtr>
static TSharedRef<SWidget> GenerateWidgetForStaticProperty(TSharedPtr<FMaterialInstanceRowData> const& RowItem)
{
	using ReturnType = TInvokeResult_T<decltype(GetterPtr), FMaterialInstanceRowData const&>;

	if constexpr(std::is_same<std::remove_cv_t<std::remove_reference_t<ReturnType>>, FSoftObjectPath>::value)
	{
		return GenerateAssetPathWidget((*GetterPtr)(*RowItem), FSlateColor::UseForeground());
	}
	else if constexpr(TIsFloatingPoint<ReturnType>::Value)
	{
		static const FNumberFormattingOptions FormatOptions = FNumberFormattingOptions().SetMaximumFractionalDigits(6);
		return SNew(STextBlock).Text(FText::AsNumber((*GetterPtr)(*RowItem), &FormatOptions)).ColorAndOpacity(GetColor((*ModifiedGetterPtr)(*RowItem)));
	}
	else
	{
		static const FNumberFormattingOptions FormatOptions = FNumberFormattingOptions().SetUseGrouping(false);
		return SNew(STextBlock).Text(FText::AsNumber((*GetterPtr)(*RowItem), &FormatOptions)).ColorAndOpacity(GetColor((*ModifiedGetterPtr)(*RowItem)));
	}
}

using PropertyGenerateWidgetFunction = TSharedRef<SWidget>(*)(TSharedPtr<FMaterialInstanceRowData> const&);

/** Struct containing information needed to generate table entries for EMaterialInstanceStaticProperties. */
struct FMaterialInstanceStaticPropertyInfo
{
	EMaterialPropertyCategory Category;
	FName ColumnId;
	FString PropertyName;
	PropertySortFunction SortFunction;
	PropertyGenerateWidgetFunction GenerateWidgetFunction;
};

/** Array for table generation data for EMaterialInstanceStaticProperties. */
static const FMaterialInstanceStaticPropertyInfo StaticPropertyInfos[(int32)EMaterialInstanceStaticProperties::Count] = {
	{EMaterialPropertyCategory::Default, "CID_AssetPath", "Asset Path", &SortByStaticProperty<&FMaterialInstanceStaticPropertyGetters::GetAssetPathIndex>, &GenerateWidgetForStaticProperty<&FMaterialInstanceStaticPropertyGetters::GetAssetPath, &FMaterialInstanceStaticPropertyModifiedGetters::GetAssetPathModified>},
	{EMaterialPropertyCategory::Default, "CID_PermutationHash", "Permutation Hash", &SortByStaticProperty<&FMaterialInstanceStaticPropertyGetters::GetPermutationHash>, &GenerateWidgetForStaticProperty<&FMaterialInstanceStaticPropertyGetters::GetPermutationHash, &FMaterialInstanceStaticPropertyModifiedGetters::GetPermutationHashModified> },
	{EMaterialPropertyCategory::StaticProperty, "CID_TwoSided", "Two Sided", &SortByStaticProperty<&FMaterialInstanceStaticPropertyGetters::GetTwoSided>, &GenerateWidgetForStaticProperty<&FMaterialInstanceStaticPropertyGetters::GetTwoSided, &FMaterialInstanceStaticPropertyModifiedGetters::GetTwoSidedModified> },
	{EMaterialPropertyCategory::StaticProperty, "CID_IsThinSurface", "Thin Surface", &SortByStaticProperty<&FMaterialInstanceStaticPropertyGetters::GetIsThinSurface>, &GenerateWidgetForStaticProperty<&FMaterialInstanceStaticPropertyGetters::GetIsThinSurface, &FMaterialInstanceStaticPropertyModifiedGetters::GetIsThinSurfaceModified> },
	{EMaterialPropertyCategory::StaticProperty, "CID_DitheredLODTransition", "Dithered LOD Transition", &SortByStaticProperty<&FMaterialInstanceStaticPropertyGetters::GetDitheredLODTransition>, &GenerateWidgetForStaticProperty<&FMaterialInstanceStaticPropertyGetters::GetDitheredLODTransition, &FMaterialInstanceStaticPropertyModifiedGetters::GetDitheredLODTransitionModified> },
	{EMaterialPropertyCategory::StaticProperty, "CID_CastDynamicShadowAsMasked", "Dynamic Shadow As Masked", &SortByStaticProperty<&FMaterialInstanceStaticPropertyGetters::GetCastDynamicShadowAsMasked>, &GenerateWidgetForStaticProperty<&FMaterialInstanceStaticPropertyGetters::GetCastDynamicShadowAsMasked, &FMaterialInstanceStaticPropertyModifiedGetters::GetCastDynamicShadowAsMaskedModified> },
	{EMaterialPropertyCategory::StaticProperty, "CID_OutputTranslucentVelocity", "Output Translucent Velocity", &SortByStaticProperty<&FMaterialInstanceStaticPropertyGetters::GetOutputTranslucentVelocity>, &GenerateWidgetForStaticProperty<&FMaterialInstanceStaticPropertyGetters::GetOutputTranslucentVelocity, &FMaterialInstanceStaticPropertyModifiedGetters::GetOutputTranslucentVelocityModified> },
	{EMaterialPropertyCategory::StaticProperty, "CID_HasPixelAnimation", "Has Pixel Animation", &SortByStaticProperty<&FMaterialInstanceStaticPropertyGetters::GetHasPixelAnimation>, &GenerateWidgetForStaticProperty<&FMaterialInstanceStaticPropertyGetters::GetHasPixelAnimation, &FMaterialInstanceStaticPropertyModifiedGetters::GetHasPixelAnimationModified> },
	{EMaterialPropertyCategory::StaticProperty, "CID_EnableTessellation", "Tessellation", &SortByStaticProperty<&FMaterialInstanceStaticPropertyGetters::GetEnableTessellation>, &GenerateWidgetForStaticProperty<&FMaterialInstanceStaticPropertyGetters::GetEnableTessellation, &FMaterialInstanceStaticPropertyModifiedGetters::GetEnableTessellationModified> },
	{EMaterialPropertyCategory::StaticProperty, "CID_BlendMode", "Blend Mode", &SortByStaticProperty<&FMaterialInstanceStaticPropertyGetters::GetBlendMode>, &GenerateWidgetForStaticProperty<&FMaterialInstanceStaticPropertyGetters::GetBlendMode, &FMaterialInstanceStaticPropertyModifiedGetters::GetBlendModeModified> },
	{EMaterialPropertyCategory::StaticProperty, "CID_ShadingModel", "Shading Model", &SortByStaticProperty<&FMaterialInstanceStaticPropertyGetters::GetShadingModel>, &GenerateWidgetForStaticProperty<&FMaterialInstanceStaticPropertyGetters::GetShadingModel, &FMaterialInstanceStaticPropertyModifiedGetters::GetShadingModelModified> },
	{EMaterialPropertyCategory::StaticProperty, "CID_OpacityMaskClipValue", "Opacity Mask Clip Value", &SortByStaticProperty<&FMaterialInstanceStaticPropertyGetters::GetOpacityMaskClipValue>, &GenerateWidgetForStaticProperty<&FMaterialInstanceStaticPropertyGetters::GetOpacityMaskClipValue, &FMaterialInstanceStaticPropertyModifiedGetters::GetOpacityMaskClipModified> },
};

/** Get column FName for a static property enumeration. */
static FName GetColumnIdFromStaticProperty(EMaterialInstanceStaticProperties InProperty)
{
	if ((int32)InProperty >= 0 && (int32)InProperty < (int32)EMaterialInstanceStaticProperties::Count)
	{
		return StaticPropertyInfos[(int32)InProperty].ColumnId;
	}
	return NAME_None;
}

/** Resolve column FName to static property enumeration. */
static EMaterialInstanceStaticProperties GetStaticPropertyFromColumnId(FName InColumnId)
{
	for (int32 Index = 0; Index < (int32)EMaterialInstanceStaticProperties::Count; ++Index)
	{
		if (StaticPropertyInfos[Index].ColumnId == InColumnId)
		{
			return (EMaterialInstanceStaticProperties)Index;
		}
	}
	return EMaterialInstanceStaticProperties::Count;
}

/** Sort row data based on the given static property enumeration. */
static void SortByStaticProperty(TArray<TSharedPtr<FMaterialInstanceRowData>>& InArray, EMaterialInstanceStaticProperties InKey, bool bInSortModifer)
{
	if (InKey != EMaterialInstanceStaticProperties::Count)
	{
		StaticPropertyInfos[(uint8)InKey].SortFunction(InArray, bInSortModifer);
	}
}

/** Generate row entry widget for the given static property enumeration. */
static TSharedRef<SWidget> GenerateWidgetForStaticProperty(TSharedPtr<FMaterialInstanceRowData> const& RowItem, EMaterialInstanceStaticProperties InKey)
{
	if (InKey != EMaterialInstanceStaticProperties::Count)
	{
		return StaticPropertyInfos[(uint8)InKey].GenerateWidgetFunction(RowItem);
	}

	return SNew(STextBlock).Text(FText::FromString("?"));
}

/** Static array of column FNames for all material usages. */
static const FName UsageColumnIds[] = {
	"CID_UsageSkeletalMesh",
	"CID_UsageParticleSprites",
	"CID_UsageBeamTrails",
	"CID_UsageMeshParticles",
	"CID_UsageStaticLighting",
	"CID_UsageMorphTargets",
	"CID_UsageSplineMesh",
	"CID_UsageInstancedStaticMeshes",
	"CID_UsageGeometryCollections",
	"CID_UsageClothing",
	"CID_UsageNiagaraSprites",
	"CID_UsageNiagaraRibbons",
	"CID_UsageNiagaraMeshParticles",
	"CID_UsageGeometryCache",
	"CID_UsageWater",
	"CID_UsageHairStrands",
	"CID_UsageLidarPointCloud",
	"CID_UsageVirtualHeightfieldMesh",
	"CID_UsageNanite",
	"CID_UsageVoxels",
	"CID_UsageCurves",
	"CID_UsageVolumetricCloud",
	"CID_UsageHeterogeneousVolumes",
	"CID_UsageStaticMesh",
	"CID_UsageEditorCompositing",
	"CID_UsageNeuralNetworks",
	"CID_UsageMeshDeformer",
	"CID_UsageInstancedSkinnedMesh",
};
static_assert(UE_ARRAY_COUNT(UsageColumnIds) == MATUSAGE_MAX, "UsageColumnIds must match EMaterialUsage. Did you add a new MATUSAGE entry?");

/** Static array of column FNames for column type hashes, outside of Default */
static const FName CategoryHashIds[(int32)(EMaterialPropertyCategory::Count)] = {
	"CID_DefaultUnused",
	"CID_StaticPropertyHash",
	"CID_UsageFlagHash",
	"CID_StaticSwitchHash",
	"CID_ComponentMaskHash",
};

/** Resolve column FName to material usage. */
static EMaterialUsage GetMaterialUsageFromColumnId(FName InColumnId)
{
	for (int32 Index = 0; Index < MATUSAGE_MAX; ++Index)
	{
		if (UsageColumnIds[Index] == InColumnId)
		{
			return (EMaterialUsage)Index;
		}
	}
	return MATUSAGE_MAX;
}

/** Sort row data based on the given material usage. */
static void SortByUsage(TArray<TSharedPtr<FMaterialInstanceRowData>>& InArray, EMaterialUsage InUsage, bool bInSortModifer)
{
	if (InUsage < MATUSAGE_MAX)
	{
		if (bInSortModifer)
		{
			Algo::StableSort(InArray, [InUsage](TSharedPtr<FMaterialInstanceRowData> const& A, TSharedPtr<FMaterialInstanceRowData> const& B)
			{
				return (A->AssetData.StaticProperties.UsageFlags & (1u << InUsage)) > (B->AssetData.StaticProperties.UsageFlags & (1u << InUsage));
			});
		}
		else
		{
			Algo::StableSort(InArray, [InUsage](TSharedPtr<FMaterialInstanceRowData> const& A, TSharedPtr<FMaterialInstanceRowData> const& B)
			{
				return (A->AssetData.StaticProperties.UsageFlags & (1u << InUsage)) < (B->AssetData.StaticProperties.UsageFlags & (1u << InUsage));
			});
		}
	}
}

/** Sort row data based on the given derived column type hash index. */
static void SortByColumnHash(TArray<TSharedPtr<FMaterialInstanceRowData>>& InArray, int32 Index, bool bInSortModifer)
{
	if (Index < (int32)EMaterialPropertyCategory::Count)
	{
		if (bInSortModifer)
		{
			Algo::StableSort(InArray, [Index](TSharedPtr<FMaterialInstanceRowData> const& A, TSharedPtr<FMaterialInstanceRowData> const& B)
			{
				return (A->PerCategoryHash[Index]) > (B->PerCategoryHash[Index]);
			});
		}
		else
		{
			Algo::StableSort(InArray, [Index](TSharedPtr<FMaterialInstanceRowData> const& A, TSharedPtr<FMaterialInstanceRowData> const& B)
			{
				return (A->PerCategoryHash[Index]) < (B->PerCategoryHash[Index]);
			});
		}
	}
}


/** Resolve column FName to index of static switch parameter. Returns INDEX_NONE if not found. */
static int32 GetStaticSwitchIndexFromColumnId(FName InColumnId, FMaterialInstanceParameterNames const& InParameterNames)
{
	int32 const* IndexPtr = InParameterNames.StaticSwitchNameToIndex.Find(InColumnId);
	return IndexPtr != nullptr ? *IndexPtr : INDEX_NONE;
}

/** Sort row data based on the given static switch parameter index. */
static void SortByStaticSwitch(TArray<TSharedPtr<FMaterialInstanceRowData>>& InArray, int32 InIndex, bool bInSortModifer)
{
	if (InIndex < 0)
	{
		return;
	}

	const uint32 BitArrayIndex = (uint32)InIndex / 32;
	const uint32 BitIndex = (uint32)InIndex % 32;
	if (bInSortModifer)
	{
		Algo::StableSort(InArray, [BitArrayIndex, BitIndex](TSharedPtr<FMaterialInstanceRowData> const& A, TSharedPtr<FMaterialInstanceRowData> const& B)
		{
			return (A->AssetData.StaticSwitchOverrideValues[BitArrayIndex] & (1u << BitIndex)) > (B->AssetData.StaticSwitchOverrideValues[BitArrayIndex] & (1u << BitIndex));
		});
	}
	else
	{
		Algo::StableSort(InArray, [BitArrayIndex, BitIndex](TSharedPtr<FMaterialInstanceRowData> const& A, TSharedPtr<FMaterialInstanceRowData> const& B)
		{
			return (A->AssetData.StaticSwitchOverrideValues[BitArrayIndex] & (1u << BitIndex)) < (B->AssetData.StaticSwitchOverrideValues[BitArrayIndex] & (1u << BitIndex));
		});
	}
}

/** Resolve column FName to index of static component mask parameter. Returns INDEX_NONE if not found. */
static int32 GetComponentMaskIndexFromColumnId(FName InColumnId, FMaterialInstanceParameterNames const& InParameterNames)
{
	int32 const* IndexPtr = InParameterNames.ComponentMaskNameToIndex.Find(InColumnId);
	return IndexPtr != nullptr ? *IndexPtr : INDEX_NONE;
}

/** Sort row data based on the given static component mask parameter index. */
static void SortByComponentMask(TArray<TSharedPtr<FMaterialInstanceRowData>>& InArray, int32 InIndex, bool bInSortModifer)
{
	if (InIndex < 0)
	{
		return;
	}

	const uint32 BitArrayIndex = InIndex / 8;
	const uint32 BitIndex = (InIndex * 4) % 32;
	if (bInSortModifer)
	{
		Algo::StableSort(InArray, [BitArrayIndex, BitIndex](TSharedPtr<FMaterialInstanceRowData> const& A, TSharedPtr<FMaterialInstanceRowData> const& B)
		{
			return ((A->AssetData.ComponentMaskOverrideValues[BitArrayIndex] >> BitIndex) & 0xf) > ((B->AssetData.ComponentMaskOverrideValues[BitArrayIndex] >> BitIndex) & 0xf);
		});
	}
	else
	{
		Algo::StableSort(InArray, [BitArrayIndex, BitIndex](TSharedPtr<FMaterialInstanceRowData> const& A, TSharedPtr<FMaterialInstanceRowData> const& B)
		{
			return ((A->AssetData.ComponentMaskOverrideValues[BitArrayIndex] >> BitIndex) & 0xf) < ((B->AssetData.ComponentMaskOverrideValues[BitArrayIndex] >> BitIndex) & 0xf);
		});
	}
}

/** A slate container of a data table entry describing a UMaterialInstance. */
class SMaterialInstanceTableRow : public SMultiColumnTableRow<TSharedPtr<FMaterialInstanceRowData>>
{
	/** Shared pointer to container of resoved static parameter names owned by the table. */
	TSharedPtr<FMaterialInstanceParameterNames> ParameterNames;
	/** The material instance stats for the row. */
	TSharedPtr<FMaterialInstanceRowData> RowItem;

public:
	SLATE_BEGIN_ARGS(SMaterialInstanceTableRow) {}
		SLATE_ARGUMENT(TSharedPtr<FMaterialInstanceParameterNames>, ParameterNames)
		SLATE_ARGUMENT(TSharedPtr<FMaterialInstanceRowData>, Item)
	SLATE_END_ARGS()

	/** Builds the row. */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable)
	{
		ParameterNames = InArgs._ParameterNames;
		RowItem = InArgs._Item;
		SMultiColumnTableRow<TSharedPtr<FMaterialInstanceRowData>>::Construct(FSuperRowType::FArguments(), InOwnerTable);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnId) override
	{
		if (RowItem.IsValid())
		{
			// Try to resolve column as static property.
			const EMaterialInstanceStaticProperties Property = GetStaticPropertyFromColumnId(ColumnId);
			if (Property != EMaterialInstanceStaticProperties::Count)
			{
				return GenerateWidgetForStaticProperty(RowItem, Property);
			}

			// Try to resolve column as material usage.
			const EMaterialUsage Usage = GetMaterialUsageFromColumnId(ColumnId);
			if (Usage != MATUSAGE_MAX)
			{
				const bool bValue = (RowItem->AssetData.StaticProperties.UsageFlags & (1u << Usage)) != 0;
				const bool bModified = (RowItem->AssetData.StaticPropertyOverrideFlags.Override_UsageFlags & (1u << Usage)) != 0;
				return SNew(STextBlock).Text(FText::AsNumber(bValue ? 1 : 0)).ColorAndOpacity(GetColor(bModified));
			}

			// Try to resolve column as static switch.
			const int32 StaticSwitchIndex = GetStaticSwitchIndexFromColumnId(ColumnId, *ParameterNames);
			if (StaticSwitchIndex != INDEX_NONE)
			{
				const uint32 BitArrayIndex = StaticSwitchIndex / 32;
				const uint32 BitIndex = StaticSwitchIndex % 32;
				const bool bValue = (RowItem->AssetData.StaticSwitchOverrideValues[BitArrayIndex] & (1 << BitIndex)) != 0;
				const bool bModified = RowItem->AssetData.StaticSwitchOverrideMask.IsValidIndex(BitArrayIndex) ? (RowItem->AssetData.StaticSwitchOverrideMask[BitArrayIndex] & (1 << BitIndex)) != 0 : false;
				return SNew(STextBlock).Text(FText::AsNumber(bValue ? 1 : 0)).ColorAndOpacity(GetColor(bModified));
			}

			// Try to resolve column as static component mask.
			const int32 ComponentMaskIndex = GetComponentMaskIndexFromColumnId(ColumnId, *ParameterNames);
			if (ComponentMaskIndex != INDEX_NONE)
			{
				const uint32 BitArrayIndex = ComponentMaskIndex / 8;
				const uint32 BitIndex = (ComponentMaskIndex * 4) % 32;
				const uint32 Value = RowItem->AssetData.ComponentMaskOverrideValues[BitArrayIndex] >> BitIndex;
				const uint32 Modified = RowItem->AssetData.ComponentMaskOverrideMask.IsValidIndex(BitArrayIndex) ? (RowItem->AssetData.ComponentMaskOverrideMask[BitArrayIndex] >> BitIndex) & 0xF : 0;
				TStringBuilder<8> RGBA;
				RGBA.AppendChar(Value & 1 ? 'R' : '_');
				RGBA.AppendChar(Value & 2 ? 'G' : '_');
				RGBA.AppendChar(Value & 4 ? 'B' : '_');
				RGBA.AppendChar(Value & 8 ? 'A' : '_');
				return SNew(STextBlock).Text(FText::FromString(*RGBA)).ColorAndOpacity(GetColor(Modified != 0));
			}

			// Try to resolve column as a column type hash
			static const FNumberFormattingOptions FormatOptions = FNumberFormattingOptions().SetUseGrouping(false);
			for (int32 Index = 0; Index < (int32)EMaterialPropertyCategory::Count; ++Index)
			{
				if (ColumnId == CategoryHashIds[Index])
				{
					return SNew(STextBlock).Text(FText::AsNumber(RowItem->PerCategoryHash[Index], &FormatOptions));
				}
			}
		}
		return SNew(STextBlock).Text(FText::FromString("?"));
	}
};

/** A slate widget containing a table of UMaterialInstance descriptions. */
class SMaterialInstanceTable : public SCompoundWidget
{
	/** All the available columns. */
	TArray<TSharedPtr<FMaterialInstanceColumnData>> AllColumns;
	/** The column ids for the currently hidden subset of AllColumns. */
	TSet<FName> HiddenColumns;

	/** All the available rows. */
	TArray<TSharedPtr<FMaterialInstanceRowData>> AllRows;
	/** An ordered and filtered subset of AllRows. */
	TArray<TSharedPtr<FMaterialInstanceRowData>> FilteredRows;
	
	/** Resolved static switch and component mask parameter names. */
	TSharedPtr<FMaterialInstanceParameterNames> ParameterNames;

	/** Table header row. */
	TSharedPtr<SHeaderRow> HeaderRow;
	/** A slate view of the table. */
	TSharedPtr<SListView<TSharedPtr<FMaterialInstanceRowData>>> ListView;

	/** Filter for category of columns to show. */
	EMaterialPropertyCategory SelectedCategory;

	/** Search text used for filtering the rows to show. */
	FString RowSearchText;
	/** Search text used for filtering the columns to show. */
	FString ColumnSearchText;
	/** True if we are filtering to unique instances. */
	bool bUniqueInstances;

	/** Selected column name for sorting the table. */
	FName SortColumnId = NAME_None;
	/** Selected mode for sorting the table. */
	EColumnSortMode::Type SortMode = EColumnSortMode::None;

public:
	SLATE_BEGIN_ARGS(SMaterialInstanceTable) {}
		SLATE_ARGUMENT(TArray<TSharedPtr<FMaterialInstanceColumnData>>, InitialColumns)
		SLATE_ARGUMENT(TArray<TSharedPtr<FMaterialInstanceRowData>>, InitialRows)
		SLATE_ARGUMENT(TSharedPtr<FMaterialInstanceParameterNames>, InitialParameterNames)
	SLATE_END_ARGS()

	/** Builds the slate table view. */
	void Construct(const FArguments& InArgs)
	{
		AllColumns = InArgs._InitialColumns;
		AllRows = InArgs._InitialRows;
		FilteredRows = InArgs._InitialRows;

		ParameterNames = InArgs._InitialParameterNames;

		SelectedCategory = EMaterialPropertyCategory::StaticProperty;

		RowSearchText.Reset();
		ColumnSearchText.Reset();
		bUniqueInstances = false;

		SortColumnId = GetColumnIdFromStaticProperty(EMaterialInstanceStaticProperties::PermutationHash);
		SortMode = EColumnSortMode::Descending;

		HeaderRow = SNew(SHeaderRow);

		ChildSlot
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot().AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot().FillWidth(0.2).Padding(2)
				[
					SNew(SComboButton)
					.OnGetMenuContent(this, &SMaterialInstanceTable::OnGetCategoryContent)
					.ButtonContent()
					[
						SNew(STextBlock)
						.Text(this, &SMaterialInstanceTable::GetSelectedCategoryText)
					]
				]
				+ SHorizontalBox::Slot().FillWidth(0.1).Padding(2)
				[
					SNew(SCheckBox)
					.OnCheckStateChanged(this, &SMaterialInstanceTable::OnUniqueCheckboxChanged)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("Toolbox_UniqueCheckbox", "Unique"))
					]
				]
				+ SHorizontalBox::Slot().FillWidth(0.4).Padding(2)
				[
					SNew(SSearchBox)
					.HintText(LOCTEXT("MaterialInstanceSearchHint", "Filter By Asset"))
					.OnTextChanged(this, &SMaterialInstanceTable::OnRowSearchTextChanged)
				]
				+SHorizontalBox::Slot().FillWidth(0.4).Padding(2)
				[
					SNew(SSearchBox)
					.HintText(LOCTEXT("ColumnSearchHint", "Filter Columns"))
					.OnTextChanged(this, &SMaterialInstanceTable::OnColumnSearchTextChanged)
				]
			]
			+SVerticalBox::Slot().HAlign(HAlign_Fill).FillHeight(1.0f)
			[
				SNew(SScrollBox)
				.Orientation(Orient_Horizontal)
				+SScrollBox::Slot()
				[
					SAssignNew(ListView, SListView<TSharedPtr<FMaterialInstanceRowData>>)
					.HeaderRow(HeaderRow.ToSharedRef())
					.ListItemsSource(&FilteredRows)
					.OnGenerateRow_Lambda([this](TSharedPtr<FMaterialInstanceRowData> InItem, const TSharedRef<STableViewBase>& InOwnerTable)
					{
						return SNew(SMaterialInstanceTableRow, InOwnerTable).Item(InItem).ParameterNames(ParameterNames);
					})
				]
			]
		];
	}

	/** Set the table content. */
	void SetContent(
		TArray<TSharedPtr<FMaterialInstanceRowData>> const& InRows,
		TArray<TSharedPtr<FMaterialInstanceColumnData>> const& InColumns,
		TSharedPtr<FMaterialInstanceParameterNames> InParameterNames)
	{
		AllRows = InRows;
		AllColumns = InColumns;
		ParameterNames = InParameterNames;

		// Reset sort column if it no longer applies
		if (FindColumnById(SortColumnId) == nullptr)
		{
			SortColumnId = GetColumnIdFromStaticProperty(EMaterialInstanceStaticProperties::PermutationHash);
		}

		UpdateHeaderRow();
		UpdateColumnFilter();
		UpdateRowFilter();
		UpdateSort();
		UpdateListView();
	}

protected:
	/** Called to fill EMaterialPropertyCategory filter combo box. */
	TSharedRef<SWidget> OnGetCategoryContent()
	{
		const UEnum* Enum = StaticEnum<EMaterialPropertyCategory>();
		FMenuBuilder MenuBuilder(true, NULL);
		for (int32 Index = 1; Index < (int32)EMaterialPropertyCategory::Count; ++Index)
		{
			FUIAction ItemAction(FExecuteAction::CreateSP(this, &SMaterialInstanceTable::OnSelectCategory, Index));
			MenuBuilder.AddMenuEntry(Enum->GetDisplayNameTextByIndex(Index), TAttribute<FText>(), FSlateIcon(), ItemAction);
		}
		return MenuBuilder.MakeWidget();
	}

	/** Called to get current EMaterialPropertyCategory filter text entry. */
	FText GetSelectedCategoryText() const
	{
		const UEnum* Enum = StaticEnum<EMaterialPropertyCategory>();
		return Enum->GetDisplayNameTextByIndex((int32)SelectedCategory);
	}
	
	/** Update the view. Must be called after any work that updates filtered/sorted content. */
	void UpdateListView()
	{
		if (ListView.IsValid())
		{
			ListView->RebuildList();
		}
	}

	/** Add all columns to header row. */
	void UpdateHeaderRow()
	{
		// Gather all columns and use the batched SHeaderRow::AddColumns() for better performance when there are many columns.
		TArray<SHeaderRow::FColumn::FArguments> AllArgs;
		AllArgs.Reserve(AllColumns.Num());

		for (int32 Index = 0; Index < AllColumns.Num(); ++Index)
		{
			TSharedPtr<FMaterialInstanceColumnData> const& Column = AllColumns[Index];
			SHeaderRow::FColumn::FArguments& Args = AllArgs.Add_GetRef(SHeaderRow::Column(Column->Id));
			
			if (Column->Id == GetColumnIdFromStaticProperty(EMaterialInstanceStaticProperties::AssetPath))
			{
				Args
				.HAlignCell(HAlign_Left)
				.ManualWidth(GAssetNameColumnWidth)
				.ManualWidth_Lambda([]() { return GAssetNameColumnWidth; })
				.OnWidthChanged_Lambda([](float NewWidth) { GAssetNameColumnWidth = NewWidth; });
			}
			else
			{
				Args
				.HAlignCell(HAlign_Center)
				.FixedWidth(120);
			}
			
			Args
			.HAlignHeader(HAlign_Center)
			.DefaultLabel(Column->Label).OnSort(this, &SMaterialInstanceTable::OnColumnSortModeChanged)
			.ShouldGenerateWidget(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &SMaterialInstanceTable::ShouldShowColumn, Column->Id)));
		}

		HeaderRow->ClearColumns();
		HeaderRow->AddColumns(AllArgs);
	}

	/** Called on selection from EMaterialPropertyCategory filter combo box. */
	void OnSelectCategory(int32 Index)
	{
		SelectedCategory = (EMaterialPropertyCategory)Index;

		UpdateRowFilter();
		UpdateColumnFilter();
		UpdateListView();
	}

	/** Called when modifying the row search text. */
	void OnRowSearchTextChanged(const FText& InText)
	{
		RowSearchText = InText.ToString();

		UpdateRowFilter();
		UpdateSort();
		UpdateListView();
	}

	/** Called when modifying the unique instances checkbox. */
	void OnUniqueCheckboxChanged(ECheckBoxState NewState)
	{
		bUniqueInstances = (NewState == ECheckBoxState::Checked);

		UpdateRowFilter();
		UpdateSort();
		UpdateListView();
	}	

	/**  Apply filter to AllRows to recreate FilteredRows. */
	void UpdateRowFilter()
	{
		FilteredRows.Reset();

		TSet<uint32> PermutationHashesSeen;

		FTextFilterExpressionEvaluator Filter(ETextFilterExpressionEvaluatorMode::BasicString);
		Filter.SetFilterText(FText::FromString(RowSearchText));

		for (const TSharedPtr<FMaterialInstanceRowData>& Row : AllRows)
		{
			if (RowSearchText.IsEmpty() || Filter.TestTextFilter(FBasicStringFilterExpressionContext(Row->AssetPath.ToString())))
			{
				if (bUniqueInstances)
				{
					bool bInSet = false;
					PermutationHashesSeen.Add(Row->PerCategoryHash[(uint32)SelectedCategory], &bInSet);
					if (bInSet)
					{
						continue;
					}
				}
				FilteredRows.Add(Row);
			}
		}
	}

	/** Called when modifying the column search text. */
	void OnColumnSearchTextChanged(const FText& InText)
	{
		ColumnSearchText = InText.ToString();

		UpdateColumnFilter();
		UpdateListView();
	}

	/**  Apply filter to AllColumns to recreate HiddenColumns and update HeaderRow. */
	void UpdateColumnFilter()
	{
		HiddenColumns.Reset();

		for (int32 Index = 0; Index < AllColumns.Num(); ++Index)
		{
			const TSharedPtr<FMaterialInstanceColumnData>& Column = AllColumns[Index];

			if (Column->Category == EMaterialPropertyCategory::Default)
			{
				// Default columns are never hidden.
				continue;
			}
				
			if (Column->Category == SelectedCategory && 
				(ColumnSearchText.IsEmpty() || Column->Label.ToString().Contains(ColumnSearchText, ESearchCase::IgnoreCase)))
			{
				continue;
			}
		
			HiddenColumns.Add(Column->Id);
		}

		HeaderRow->RefreshColumns();
	}

	/** UI delegate to determine if a column should be shown. */
	bool ShouldShowColumn(FName ColumnId)
	{
		return !HiddenColumns.Contains(ColumnId);
	}

	/** Called when clicking on a column header to sort. */
	void OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode)
	{
		if (SortColumnId != ColumnId)
		{
			// Initial sort mode is ascending for asset path, descending for numeric.
			SortMode = ColumnId == GetColumnIdFromStaticProperty(EMaterialInstanceStaticProperties::AssetPath) ? EColumnSortMode::Ascending : EColumnSortMode::Descending;
			SortColumnId = ColumnId;
		}
		else
		{
			SortMode = SortMode == EColumnSortMode::Descending ? EColumnSortMode::Ascending : EColumnSortMode::Descending;
		}

		UpdateSort();
		UpdateListView();
	}

	/**  Find column, if any. */
	TSharedPtr<FMaterialInstanceColumnData>* FindColumnById(FName Id)
	{
		return AllColumns.FindByPredicate([ColumnId = Id](TSharedPtr<FMaterialInstanceColumnData> const& ColumnData)
				{
					return ColumnData->Id == ColumnId; 
				});	
	}

	/**  Apply sort to FilteredRows. */
	void UpdateSort()
	{
		if (AllColumns.IsEmpty() || AllRows.IsEmpty())
		{
			// If the panel is empty, there is nothing to sort
			return;
		}

		if (HiddenColumns.Contains(SortColumnId))
		{
			// If we are filtering out the sort column then the current behavior is no sorting until a column to sort is next selected.
			return;
		}

		TSharedPtr<FMaterialInstanceColumnData>* SortColumnData = FindColumnById(SortColumnId);
		if (!ensure(SortColumnData != nullptr))
		{
			return;
		}

		// Call specific sorting function for the column type.
		const bool bSortModifier = SortMode == EColumnSortMode::Descending;
		const EMaterialPropertyCategory Category = (*SortColumnData)->Category;

		// See if we are sorting from a column type hash
		for (int32 Index = 0; Index < (int32)EMaterialPropertyCategory::Count; ++Index)
		{
			if (CategoryHashIds[Index] == SortColumnId)
			{
				SortByColumnHash(FilteredRows, Index, bSortModifier);
				return;
			}
		}

		switch (Category)
		{
			case EMaterialPropertyCategory::Default:
			case EMaterialPropertyCategory::StaticProperty:
				SortByStaticProperty(FilteredRows, GetStaticPropertyFromColumnId(SortColumnId), bSortModifier);
				break;
			case EMaterialPropertyCategory::UsageFlag:
				SortByUsage(FilteredRows, GetMaterialUsageFromColumnId(SortColumnId), bSortModifier);
				break;
			case EMaterialPropertyCategory::StaticSwitch:
				SortByStaticSwitch(FilteredRows, GetStaticSwitchIndexFromColumnId(SortColumnId, *ParameterNames), bSortModifier);
			break;
			case EMaterialPropertyCategory::ComponentMask:
				SortByComponentMask(FilteredRows, GetComponentMaskIndexFromColumnId(SortColumnId, *ParameterNames), bSortModifier);
				break;
		}
	}
};

/** Instantiate a table widget for all material instances in a material. */
static TSharedPtr<SMaterialInstanceTable> CreateMaterialInstanceTable()
{
	TArray<TSharedPtr<FMaterialInstanceColumnData>> Columns;
	TArray<TSharedPtr<FMaterialInstanceRowData>> Rows;
	TSharedPtr<FMaterialInstanceParameterNames> ParameterNames = MakeShared<FMaterialInstanceParameterNames>();

	return SNew(SMaterialInstanceTable).InitialColumns(Columns).InitialRows(Rows).InitialParameterNames(ParameterNames);
}

struct FExperimentKey
{
	FStaticPermutationProperties StaticProperties;
	TArray<uint32> SwitchValues;
	TArray<uint32> MaskValues;
	uint32 MaterialLayerHash;
};

static FExperimentKey MakeExperimentKey(FMaterialValidationAssetData_MaterialInstance const& AssetData)
{
	return { AssetData.StaticProperties, AssetData.StaticSwitchOverrideValues, AssetData.ComponentMaskOverrideValues, AssetData.MaterialLayerHash };
}

static uint32 HashExperimentKey(FExperimentKey const& Key)
{
	return MaterialValidation::BuildPermutationHash(Key.StaticProperties, Key.SwitchValues, Key.MaskValues, Key.MaterialLayerHash);
}

using FExperimentKeyModifier = TFunction<FExperimentKey(FExperimentKey const&)>;

static void RunExperiment(
	FString const& Name,
	TArray<TSharedPtr<FMaterialInstanceRowData>> const& InstanceRows,
	int32 BaselineCount,
	TArray<TSharedPtr<FExperimentData>>& OutputRows,
	FExperimentKeyModifier Modifier)
{
	if (BaselineCount > 0 && InstanceRows.Num())
	{
		TSet<uint32> HypotheticalHashes;
		for (auto const& Row : InstanceRows)
		{
			HypotheticalHashes.Add(HashExperimentKey(Modifier(MakeExperimentKey(Row->AssetData))));
		}

		const uint32 Reduction = FMath::Max(0, BaselineCount - (int32)HypotheticalHashes.Num());

		if (Reduction > 0)
		{
			auto Result = MakeShared<FExperimentData>();
			Result->Name = Name;
			Result->ReductionCount = Reduction;
			Result->ReductionPercent = (float)Reduction / BaselineCount;
			OutputRows.Add(Result);
		}
	}
}

/** Update the table for all material instances in a material. */
static void UpdateMaterialInstanceTables(
	TSharedPtr<SMaterialInstanceTable> InMaterialTable,
	TSharedPtr<SExperimentsTable> InExperimentsTable,
	UMaterialValidationGroup const* InGroup, 
	FSoftObjectPath const& InMaterialPath)
{
	// Get all the information from the material validation database.
	FMaterialDatabaseAssetHierarchyInfo AssetInfo;
	UMaterialValidationLibrary::GetMaterialHierarchyInfo(InGroup, InMaterialPath, AssetInfo);

	// Build all rows.
	TArray<TSharedPtr<FMaterialInstanceRowData>> Rows;
	const int32 NumMaterialInstances = AssetInfo.MaterialPaths.Num();
	for (int32 Index = 0; Index < NumMaterialInstances; ++Index)
	{
		Rows.Add(MakeShared<FMaterialInstanceRowData>(AssetInfo.MaterialPaths[Index], Index, AssetInfo.MaterialAssetDatas[Index]));
	}

	// Build all columns.
	TArray<TSharedPtr<FMaterialInstanceColumnData>> Columns;
	if (Rows.Num())
	{
		// Build path and permutation hash so they are first
		for (int32 PropertyIndex = 0; PropertyIndex < 2; ++PropertyIndex)
		{
			Columns.Add(MakeShared<FMaterialInstanceColumnData>(StaticPropertyInfos[PropertyIndex].ColumnId, FText::FromString(StaticPropertyInfos[PropertyIndex].PropertyName), StaticPropertyInfos[PropertyIndex].Category));
		}

		// Build additional type-specific hash columns
		Columns.Add(MakeShared<FMaterialInstanceColumnData>(CategoryHashIds[(int32)EMaterialPropertyCategory::StaticProperty], LOCTEXT("Toolkit_StaticPropertyHash","Static Property Hash"), EMaterialPropertyCategory::StaticProperty));
		Columns.Add(MakeShared<FMaterialInstanceColumnData>(CategoryHashIds[(int32)EMaterialPropertyCategory::UsageFlag], LOCTEXT("Toolkit_UsageHash","Usage Hash"), EMaterialPropertyCategory::UsageFlag));
		Columns.Add(MakeShared<FMaterialInstanceColumnData>(CategoryHashIds[(int32)EMaterialPropertyCategory::StaticSwitch], LOCTEXT("Toolkit_StaticSwitchHash","Static Switch Hash"), EMaterialPropertyCategory::StaticSwitch));
		Columns.Add(MakeShared<FMaterialInstanceColumnData>(CategoryHashIds[(int32)EMaterialPropertyCategory::ComponentMask], LOCTEXT("Toolkit_ComponentMaskHash","Component Mask Hash"), EMaterialPropertyCategory::ComponentMask));

		// Build the rest of the columns
		for (int32 PropertyIndex = 2; PropertyIndex < (int32)EMaterialInstanceStaticProperties::Count; ++PropertyIndex)
		{
			Columns.Add(MakeShared<FMaterialInstanceColumnData>(StaticPropertyInfos[PropertyIndex].ColumnId, FText::FromString(StaticPropertyInfos[PropertyIndex].PropertyName), StaticPropertyInfos[PropertyIndex].Category));
		}
		for (int32 UsageIndex = 0; UsageIndex < MATUSAGE_MAX; ++UsageIndex)
		{
			Columns.Add(MakeShared<FMaterialInstanceColumnData>(UsageColumnIds[UsageIndex], FText::FromString(UMaterialInterface::GetUsageName((EMaterialUsage)UsageIndex)), EMaterialPropertyCategory::UsageFlag));
		}
		for (FName& Name : AssetInfo.StaticSwitchNames)
		{
			Columns.Add(MakeShared<FMaterialInstanceColumnData>(Name, FText::FromName(Name), EMaterialPropertyCategory::StaticSwitch));
		}
		for (FName& Name : AssetInfo.ComponentMaskNames)
		{
			Columns.Add(MakeShared<FMaterialInstanceColumnData>(Name, FText::FromName(Name), EMaterialPropertyCategory::ComponentMask));
		}
	}
	
	// Move parameter names and build FMaterialInstanceParameterNames lookups.
	TSharedPtr<FMaterialInstanceParameterNames> ParameterNames = MakeShared<FMaterialInstanceParameterNames>();

	ParameterNames->StaticSwitches = MoveTemp(AssetInfo.StaticSwitchNames);
	ParameterNames->ComponentMasks = MoveTemp(AssetInfo.ComponentMaskNames);

	for (int32 Index = 0; Index < ParameterNames->StaticSwitches.Num(); ++Index)
	{
		ParameterNames->StaticSwitchNameToIndex.Add(ParameterNames->StaticSwitches[Index], Index);
	}
	for (int32 Index = 0; Index < ParameterNames->ComponentMasks.Num(); ++Index)
	{
		ParameterNames->ComponentMaskNameToIndex.Add(ParameterNames->ComponentMasks[Index], Index);
	}

	// Apply to the table.
	InMaterialTable->SetContent(Rows, Columns, ParameterNames);

	// do Experiments
	TArray<TSharedPtr<FExperimentData>> ExperimentRows;

	if (Rows.Num())
	{
		TSet<uint32> BaselineHashes;
		for (auto const& Row : Rows)
		{ 
			BaselineHashes.Add(HashExperimentKey(MakeExperimentKey(Row->AssetData)));
		}
		const int32 BaselineCount = BaselineHashes.Num();

		RunExperiment(TEXT("No-op"), Rows, BaselineCount, ExperimentRows,
			[](FExperimentKey Key) { return Key; });

		// No-op should add no rows
		ensure(ExperimentRows.Num() == 0);

		RunExperiment(TEXT("Normalize OpacityMaskClipValue"), Rows, BaselineCount, ExperimentRows,
			[](FExperimentKey Key) { Key.StaticProperties.OpacityMaskClipValue = 0.f; return Key; });

		for (int32 SwitchIndex = 0; SwitchIndex < ParameterNames->StaticSwitches.Num(); ++SwitchIndex)
		{
			RunExperiment(
				FString::Printf(TEXT("Eliminate Switch %s"), *ParameterNames->StaticSwitches[SwitchIndex].ToString()),
				Rows, BaselineCount, ExperimentRows,
				[SwitchIndex](FExperimentKey Key)
				{
					const uint32 DwordIndex = SwitchIndex / 32;
					const uint32 BitIndex = SwitchIndex % 32;
					if (DwordIndex < (uint32)Key.SwitchValues.Num())
					{
						Key.SwitchValues[DwordIndex] &= ~(1u << BitIndex);
					}
					return Key;
				});
		}

		if (!ParameterNames->ComponentMasks.IsEmpty())
		{
			// Elimination of single-channel component masks, these all can be ChannelMaskParameter which has no permutation cost
			TArray<uint32> const& BaseMaskValues = Rows[0]->AssetData.ComponentMaskOverrideValues;
			const int32 NumMasks = ParameterNames->ComponentMasks.Num();

			TArray<uint32> SingleChannelClearMask;
			SingleChannelClearMask.SetNumZeroed(BaseMaskValues.Num());

			for (int32 Index = 0; Index < NumMasks; ++Index)
			{
				const uint32 DwordIndex = Index / 8;
				const uint32 BitIndex = (Index * 4) % 32;
				if (DwordIndex < (uint32)BaseMaskValues.Num())
				{
					const uint32 Nibble = (BaseMaskValues[DwordIndex] >> BitIndex) & 0xf;
					const bool bSingleChannel = Nibble != 0 && (Nibble & (Nibble - 1)) == 0;
					if (bSingleChannel)
					{
						SingleChannelClearMask[DwordIndex] |= (0xfu << BitIndex);
					}
				}
			}

			RunExperiment(TEXT("Single-channel Component Masks to ChannelMaskParameter"), Rows, BaselineCount, ExperimentRows,
				[SingleChannelClearMask](FExperimentKey Key)
				{
					for (int32 i = 0; i < Key.MaskValues.Num() && i < SingleChannelClearMask.Num(); ++i)
					{
						Key.MaskValues[i] &= ~SingleChannelClearMask[i];
					}
					return Key;
				});
		}
	}

	// sort by best percentage win
	ExperimentRows.Sort([](TSharedPtr<FExperimentData> const& A, TSharedPtr<FExperimentData> const& B)
		{
			return A->ReductionPercent > B->ReductionPercent;
		});

	InExperimentsTable->SetContent(ExperimentRows);
}

} // namespace MaterialValidation

/**
 * AssetEditorToolkit for UMaterialValidationGroup.
 * This sets up the tabs for showing various table views of the data in the group.
 */
class FMaterialValidationToolkit : public FAssetEditorToolkit, public FGCObject, public FNotifyHook
{
	/** The current UMaterialValidationGroup being edited. */
	TObjectPtr<UMaterialValidationGroup> EditGroup;

	/** Menu category for the toolkit tabs. */
	TSharedPtr<class FWorkspaceItem> WorkspaceMenuCategory;

	/** Standard details panel view. */
	TSharedPtr<IDetailsView> DetailsView;
	/** Material table widget. */
	TSharedPtr<SWidget> MaterialTable;
	/** Material instance table widget. */
	TSharedPtr<MaterialValidation::SMaterialInstanceTable> MaterialInstanceTable;
	/** Experiments table widget. */
	TSharedPtr<MaterialValidation::SExperimentsTable> ExperimentsTable;
	/** Changelists tab widget. */
	TSharedPtr<SWidget> ChangelistsTab;

	// Toolkit tabs are referenced by FName.
	static const FName TabId_Details;
	static const FName TabId_Materials;
	static const FName TabId_MaterialInstances;
	static const FName TabId_Experiments;
	static const FName TabId_Changelists;

public:
	// Begin FGCObject overrides.
	virtual FString GetReferencerName() const override
	{
		return TEXT("FMaterialValidationToolkit");
	}

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		Collector.AddReferencedObject(EditGroup);
	}
	// End FGCObject overrides.

	// Begin FAssetEditorToolkit overrides.
	virtual FName GetToolkitFName() const override
	{
		return FName("MaterialValidation");
	}

	virtual FText GetBaseToolkitName() const override
	{
		return LOCTEXT("Toolkit_AppLabel", "MaterialValidation");
	}

	virtual FString GetWorldCentricTabPrefix() const override
	{
		return LOCTEXT("Toolkit_TabPrefix", "MaterialValidation").ToString();
	}

	virtual FLinearColor GetWorldCentricTabColorScale() const override
	{
		return FLinearColor(0.0f, 0.0f, 0.2f, 0.5f);
	}

	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager) override
	{
		WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("Toolkit_WorkspaceMenuCatgory", "Material Validation"));

		InTabManager->RegisterTabSpawner(TabId_Details, FOnSpawnTab::CreateSP(this, &FMaterialValidationToolkit::SpawnDetailsTab))
			.SetDisplayName(LOCTEXT("Toolkit_DetailsTab_DisplayName", "Details"))
			.SetGroup(WorkspaceMenuCategory.ToSharedRef())
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

		InTabManager->RegisterTabSpawner(TabId_Materials, FOnSpawnTab::CreateSP(this, &FMaterialValidationToolkit::SpawnMaterialsTab))
			.SetDisplayName(LOCTEXT("Toolkit_MaterialsTab_DisplayName", "Materials"))
			.SetGroup(WorkspaceMenuCategory.ToSharedRef())
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.StatsViewer"));

		InTabManager->RegisterTabSpawner(TabId_MaterialInstances, FOnSpawnTab::CreateSP(this, &FMaterialValidationToolkit::SpawnMaterialInstancesTab))
			.SetDisplayName(LOCTEXT("Toolkit_MaterialInstancesTab_DisplayName", "Material Instances"))
			.SetGroup(WorkspaceMenuCategory.ToSharedRef())
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.StatsViewer"));

		InTabManager->RegisterTabSpawner(TabId_Experiments, FOnSpawnTab::CreateSP(this, &FMaterialValidationToolkit::SpawnExperimentsTab))
			.SetDisplayName(LOCTEXT("Toolkit_ExperimentsTab_DisplayName", "Experiments"))
			.SetGroup(WorkspaceMenuCategory.ToSharedRef())
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.StatsViewer"));

		InTabManager->RegisterTabSpawner(TabId_Changelists, FOnSpawnTab::CreateSP(this, &FMaterialValidationToolkit::SpawnChangelistsTab))
			.SetDisplayName(LOCTEXT("Toolkit_ChangelistsTab_DisplayName", "Changelists"))
			.SetGroup(WorkspaceMenuCategory.ToSharedRef())
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.StatsViewer"));
	}

	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager) override
	{
		FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);
		InTabManager->UnregisterTabSpawner(TabId_Details);
		InTabManager->UnregisterTabSpawner(TabId_Materials);
		InTabManager->UnregisterTabSpawner(TabId_MaterialInstances);
		InTabManager->UnregisterTabSpawner(TabId_Experiments);
		InTabManager->UnregisterTabSpawner(TabId_Changelists);
	}
	// End FAssetEditorToolkit overrides.

	TSharedRef<SDockTab> SpawnDetailsTab(const FSpawnTabArgs& Args)
	{
		return SNew(SDockTab)
			.Label(LOCTEXT("Toolkit_DetailsTab_Label", "Details"))
			[
				DetailsView.ToSharedRef()
			];
	}

	TSharedRef<SDockTab> SpawnMaterialsTab(const FSpawnTabArgs& Args)
	{
		return SNew(SDockTab)
			.Label(LOCTEXT("Toolkit_Materials_Label", "Materials"))
			[
				MaterialTable.ToSharedRef()
			];
	}

	TSharedRef<SDockTab> SpawnExperimentsTab(const FSpawnTabArgs& Args)
	{
		return SNew(SDockTab)
		.Label(LOCTEXT("Toolkit_Experiments_Label", "Experiments"))
		[
			ExperimentsTable.ToSharedRef()
		];
	}

	TSharedRef<SDockTab> SpawnMaterialInstancesTab(const FSpawnTabArgs& Args)
	{
		return SNew(SDockTab)
			.Label(LOCTEXT("Toolkit_MaterialInstances_Label", "Material Instances"))
			[
				MaterialInstanceTable.ToSharedRef()
			];
	}

	TSharedRef<SDockTab> SpawnChangelistsTab(FSpawnTabArgs const& Args)
	{
		return SNew(SDockTab)
			.Label(LOCTEXT("Toolkit_Changelists_Label", "Changelists"))
			[
				ChangelistsTab.ToSharedRef()
			];
	}

	/** Populate the toolkit layout. */
	void Initialize(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UMaterialValidationGroup* InGroup)
	{
		EditGroup = InGroup;

		FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

		FDetailsViewArgs Args;
		Args.bHideSelectionTip = true;
		Args.NotifyHook = this;
		DetailsView = PropertyEditorModule.CreateDetailView(Args);
		DetailsView->SetObject(EditGroup);

		MaterialTable = MaterialValidation::CreateMaterialTable(EditGroup, MaterialValidation::FOnRowItemSelect::CreateSP(this, &FMaterialValidationToolkit::OnRowItemSelect));

		MaterialInstanceTable = MaterialValidation::CreateMaterialInstanceTable();
		ExperimentsTable = MaterialValidation::CreateExperimentsTable();
		ChangelistsTab = MaterialValidation::CreateChangelistTab();

		const TSharedRef<FTabManager::FLayout> Layout = FTabManager::NewLayout("MaterialValidationGroupLayout_v7")
			->AddArea
			(
				FTabManager::NewPrimaryArea()->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)
					->Split(FTabManager::NewStack()
						->AddTab(TabId_Details, ETabState::OpenedTab)
						->SetForegroundTab(TabId_Details))
					->Split(FTabManager::NewStack()
						->AddTab(TabId_Materials, ETabState::OpenedTab)
						->AddTab(TabId_Changelists, ETabState::ClosedTab))
					->Split(FTabManager::NewStack()
						->AddTab(TabId_MaterialInstances, ETabState::ClosedTab)
						->AddTab(TabId_Experiments, ETabState::ClosedTab))
				)
			);

		InitAssetEditor(Mode, InitToolkitHost, GetToolkitFName(), Layout, true /*bCreateDefaultStandaloneMenu*/, true /*bCreateDefaultToolbar*/, EditGroup);
	}

	/** Apply selected material to the material instances table. */
	void OnRowItemSelect(TSharedPtr<MaterialValidation::FMaterialStats> InMaterialStats)
	{
		MaterialValidation::UpdateMaterialInstanceTables(MaterialInstanceTable, ExperimentsTable, EditGroup, InMaterialStats->AssetPath);

		TSharedPtr<SDockTab> InstancesTab = GetTabManager()->FindExistingLiveTab(TabId_MaterialInstances);
		TSharedPtr<SDockTab> ExperimentsTab = GetTabManager()->FindExistingLiveTab(TabId_Experiments);

		const bool bExperimentsForeground = ExperimentsTab.IsValid() && ExperimentsTab->IsForeground();

		// Open any closed tabs
		if (!InstancesTab.IsValid()) { InvokeTab(TabId_MaterialInstances); }
		if (!ExperimentsTab.IsValid()) { InvokeTab(TabId_Experiments); }

		// Restore foreground — Experiments if it was active, Instances otherwise
		InvokeTab(bExperimentsForeground ? TabId_Experiments : TabId_MaterialInstances);
	}
};

const FName FMaterialValidationToolkit::TabId_Details("MainTab");
const FName FMaterialValidationToolkit::TabId_Materials("MaterialsTab");
const FName FMaterialValidationToolkit::TabId_MaterialInstances("MaterialInstancesTab");
const FName FMaterialValidationToolkit::TabId_Experiments("ExperimentsTab");
const FName FMaterialValidationToolkit::TabId_Changelists("ChangelistsTab");

FText UAssetDefinition_MaterialValidationGroup::GetAssetDisplayName() const 
{
	return LOCTEXT("Asset_DisplayName", "Material Validation Group"); 
}

FLinearColor UAssetDefinition_MaterialValidationGroup::GetAssetColor() const 
{
	return FLinearColor::Gray; 
}

TSoftClassPtr<UObject> UAssetDefinition_MaterialValidationGroup::GetAssetClass() const 
{
	return UMaterialValidationGroup::StaticClass(); 
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_MaterialValidationGroup::GetAssetCategories() const
{
	static TArray<FAssetCategoryPath> Categories = { EAssetCategoryPaths::Material };
	return Categories;
}

EAssetCommandResult UAssetDefinition_MaterialValidationGroup::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	for (UMaterialValidationGroup* Group : OpenArgs.LoadObjects<UMaterialValidationGroup>())
	{
		TSharedRef<FMaterialValidationToolkit> NewToolkit(new FMaterialValidationToolkit());
		NewToolkit->Initialize(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, Group);
	}

	return EAssetCommandResult::Handled;
}

EAssetCommandResult UAssetDefinition_MaterialValidationGroup::PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const
{
	UMaterialValidationGroup const* OldGroup = Cast<UMaterialValidationGroup>(DiffArgs.OldAsset);
	UMaterialValidationGroup const* NewGroup = Cast<UMaterialValidationGroup>(DiffArgs.NewAsset);

	if (OldGroup == nullptr && NewGroup == nullptr)
	{
		return EAssetCommandResult::Unhandled;
	}

	SMaterialValidationGroupDiff::CreateDiffWindow(OldGroup, NewGroup, DiffArgs.OldRevision, DiffArgs.NewRevision);
	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
