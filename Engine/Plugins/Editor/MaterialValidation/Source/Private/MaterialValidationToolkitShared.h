// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateColor.h"

class SWidget;

namespace MaterialValidation
{
	/** Shared column width for asset-path columns across all tables in the toolkit. */
	extern float GAssetNameColumnWidth;
	/** Text to use for empty cells. */
	extern const FText GEmptyText;
	/** Text to use for unknown values. */
	extern const FText GUnknownText;

	/** Creates a path display widget with a browse-to-asset button. */
	TSharedRef<SWidget> GenerateAssetPathWidget(FSoftObjectPath InAssetPath, FSlateColor InColor);
	/** Get color to use according to whether cost delta is positive, negative or neutral. */
	FSlateColor GetDeltaColor(int32 InDelta);
	/** Return FText that formats the number with a positive/negative sign. */
	FText GetTextSignedInt(int32 Value);
}
