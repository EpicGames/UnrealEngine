// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialValidators.h"

#include "Editor.h"
#include "DiffUtils.h"
#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "ISourceControlRevision.h"
#include "ISourceControlState.h"
#include "Logging/TokenizedMessage.h"
#include "MaterialEditingLibrary.h"
#include "Materials/Material.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialInstanceConstant.h"
#include "MaterialValidationConfig.h"
#include "MaterialValidationGroup.h"
#include "MaterialValidationLibrary.h"
#include "MaterialValidationSimilarityBrowser.h"
#include "Misc/DataValidation.h"
#include "Misc/PackageName.h"
#include "SourceControlOperations.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialValidators)

#define LOCTEXT_NAMESPACE "MaterialValidation"

bool UEditorValidator_MaterialPermutation::CanValidateAsset_Implementation(const FAssetData& InAssetData, UObject* InAsset, FDataValidationContext& InContext) const
{
	const EDataValidationUsecase Usecase = InContext.GetValidationUsecase();
	if (Usecase != EDataValidationUsecase::Commandlet && Usecase != EDataValidationUsecase::PreSubmit)
	{
		return false;
	}
	if (Cast<UMaterialInstanceConstant>(InAsset) == nullptr && Cast<UMaterial>(InAsset) == nullptr && Cast<UMaterialFunction>(InAsset) == nullptr)
	{
		return false;
	}
	return true;
}

EDataValidationResult UEditorValidator_MaterialPermutation::ValidateLoadedAsset_Implementation(const FAssetData& InAssetData, UObject* InAsset, FDataValidationContext& InContext)
{
	UMaterial* Material = Cast<UMaterial>(InAsset);
	UMaterialInstanceConstant* MaterialInstance = Cast<UMaterialInstanceConstant>(InAsset);
	UMaterialFunction* MaterialFunction = Cast<UMaterialFunction>(InAsset);

	if (Material || MaterialInstance)
	{
		int32 NumShaders = 0;

		TArray<UMaterialValidationGroup*> Groups;
		UMaterialValidationLibrary::GetAllGroups(Groups, /*bInSyncLoad*/true);

		for (UMaterialValidationGroup* Group : Groups)
		{
			if (Material)
			{
				bool bIsInGroupPath = false;
				bool bIsInGroup = false;
				UMaterialValidationLibrary::IsMaterialInGroup(Group, Material, bIsInGroupPath, bIsInGroup);
				
				if (bIsInGroupPath)
				{
					BaseMaterials.AddUnique(Material);
					ModifiedObjects.Add(Material);

					if (!bIsInGroup)
					{
						NumShaders = UMaterialEditingLibrary::GetNumShaderTypes(Material);
					}
					break;
				}
			}
			
			if (MaterialInstance && MaterialInstance->bHasStaticPermutationResource)
			{
				bool bMaterialInGroup = false;
				bool bMaterialPermutationInGroup = false;
				UMaterialValidationLibrary::IsMaterialInstanceInGroup(Group, MaterialInstance, bMaterialInGroup, bMaterialPermutationInGroup);

				if (bMaterialInGroup) 
				{
					BaseMaterials.AddUnique(MaterialInstance->GetMaterial());
					ModifiedObjects.Add(MaterialInstance);

					if (!bMaterialPermutationInGroup)
					{
						NumShaders = UMaterialEditingLibrary::GetNumShaderTypes(MaterialInstance);
					}
					break;
				}
			}
		}

		if (NumShaders > 0)
		{
			AssetMessage(EMessageSeverity::Info)->AddText(FText::Format(LOCTEXT("ShaderBudget_Info", "New permutation with {0} shaders"), FText::AsNumber(NumShaders)));
		}
	}

	if (MaterialFunction)
	{
		bHasFunctionFiles = true;

		// Add the base materials that use the material function for later valiation.
		// Note this will require loading UMaterial assets which could be slow for very commonly used material functions.
		TArray<FAssetData> MaterialAssetDatas;
		UMaterialEditingLibrary::GetMaterialsReferencingFunction(MaterialFunction, MaterialAssetDatas);
		for (FAssetData const& AssetData : MaterialAssetDatas)
		{
			if (UMaterial* BaseMaterial = Cast<UMaterial>(AssetData.GetAsset()))
			{
				BaseMaterials.AddUnique(BaseMaterial);
			}
		}
	}

	if (GetValidationResult() != EDataValidationResult::Invalid)
	{
		AssetPasses(InAsset);
	}

	return GetValidationResult();
}

void UEditorValidator_MaterialPermutation::PostAssetValidation(TArray<TSharedRef<FTokenizedMessage>>& OutMessages)
{
	// Now that we have all the touched materials for the changelist we can get the shader delta per base material hierarchy.
	// We sum these to get the final shader delta to report for validation.
	int32 NumShaders = 0;

	UMaterialValidationConfig const* Config = GetDefault<UMaterialValidationConfig>();
	const int32 ShowThreshold = Config->ShaderBudgetShowThreshold;
	const int32 ApprovalThreshold = Config->ShaderBudgetApprovalThreshold;

	// Only do the work of calculating the shader delta if we have one of the thresholds set.
	if (ShowThreshold > 0 || ApprovalThreshold > 0)
	{
		TArray<UMaterialValidationGroup*> Groups;
		UMaterialValidationLibrary::GetAllGroups(Groups, /*bInSyncLoad*/true);

		// Always do fast validation which compares modified workspace objects against the stored database baseline.
		for (UMaterial* BaseMaterial : BaseMaterials)
		{
			for (UMaterialValidationGroup* Group : Groups)
			{
				bool bIsInGroupPath = false;
				bool bIsInGroup = false;
				UMaterialValidationLibrary::IsMaterialInGroup(Group, BaseMaterial, bIsInGroupPath, bIsInGroup);

				if (bIsInGroupPath)
				{
					NumShaders -= UMaterialValidationLibrary::GetShaderCount(Group, BaseMaterial);
					NumShaders += UMaterialValidationLibrary::GetModifiedShaderCount(Group, BaseMaterial, MutableView(ModifiedObjects), {}, false /*bForceLoadObjects*/);
				}
			}
		}

		// If the fast result already reaches the approval threshold, upgrade to slow validation (depot HEAD vs workspace) for a more accurate delta before emitting the message.
		// Skip when material functions are involved: their changes cannot be fully captured so the depot comparison would not reflect the true delta.
		if (ApprovalThreshold > 0 && NumShaders >= ApprovalThreshold && !bHasFunctionFiles && ISourceControlModule::Get().IsEnabled())
		{
			FMaterialValidationHelpers::FetchRevisionHistory(MutableView(ModifiedObjects));

			TArray<UMaterialInterface*> ReplacementObjects;
			TArray<UPackage*> DepotPackages;
			FMaterialValidationHelpers::LoadDepotVersions(MutableView(ModifiedObjects), ReplacementObjects, DepotPackages);

			int32 SlowNumShaders = 0;
			for (UMaterial* BaseMaterial : BaseMaterials)
			{
				for (UMaterialValidationGroup* Group : Groups)
				{
					bool bIsInGroupPath = false;
					bool bIsInGroup = false;
					UMaterialValidationLibrary::IsMaterialInGroup(Group, BaseMaterial, bIsInGroupPath, bIsInGroup);
					if (bIsInGroupPath)
					{
						SlowNumShaders -= UMaterialValidationLibrary::GetModifiedShaderCount(Group, BaseMaterial, MutableView(ModifiedObjects), ReplacementObjects, true /*bForceLoadObjects*/);
						SlowNumShaders += UMaterialValidationLibrary::GetModifiedShaderCount(Group, BaseMaterial, MutableView(ModifiedObjects), {}, true /*bForceLoadObjects*/);
					}
				}
			}

			for (UPackage* DepotPackage : DepotPackages)
			{
				DepotPackage->MarkAsGarbage();
			}

			NumShaders = SlowNumShaders;
		}
	}

	if (ShowThreshold > 0 && FMath::Abs(NumShaders) >= ShowThreshold)
	{
		FNumberFormattingOptions NumberFormat;
		NumberFormat.UseGrouping = false;
		OutMessages.Add(FTokenizedMessage::Create(EMessageSeverity::Info, FText::Format(INVTEXT("ShaderBudgetDelta: {0}"), FText::AsNumber(NumShaders, &NumberFormat))));
	}

	if (ApprovalThreshold > 0 && NumShaders >= ApprovalThreshold)
	{
		OutMessages.Add(FTokenizedMessage::Create(EMessageSeverity::Info, INVTEXT("ShaderBudgetIntegrationRequest")));
	}

	BaseMaterials.Reset();
	ModifiedObjects.Reset();
	bHasFunctionFiles = false;
}

bool UMaterialEditorValidator_Permutation::Validate(UMaterialInterface* InMaterial, FMaterialEditorValidatorContext& InContext) const
{
	if (UMaterialInstanceConstant* MaterialInstance = Cast<UMaterialInstanceConstant>(InMaterial))
	{
		TArray<UMaterialValidationGroup*> Groups;
		UMaterialValidationLibrary::GetAllGroups(Groups, /*bInSyncLoad*/false);

		for (UMaterialValidationGroup const* Group : Groups)
		{
			bool bIsMaterialInstanceInGroup = false;
			bool bIsMaterialPermutationInGroup = false;
			UMaterialValidationLibrary::IsMaterialInstanceInGroup(Group, MaterialInstance, bIsMaterialInstanceInGroup, bIsMaterialPermutationInGroup);

			if (bIsMaterialPermutationInGroup)
			{
				continue;
			}

			UMaterial* BaseMaterial = MaterialInstance->GetMaterial();

			bool bIsBaseMaterialInGroupPath = false;
			bool bIsBaseMaterialInGroup = false;
			UMaterialValidationLibrary::IsMaterialInGroup(Group, BaseMaterial, bIsBaseMaterialInGroupPath, bIsBaseMaterialInGroup);

			if (!bIsBaseMaterialInGroupPath)
			{
				continue;
			}

			// bHasStaticPermutation will be false if the material instance has no overrides.
			// If we get here with that condition and the base material is in the group then there are no shaders to count.
			const bool bHasStaticPermutation = MaterialInstance->bHasStaticPermutationResource;
			if (!bHasStaticPermutation && bIsBaseMaterialInGroup)
			{
				continue;
			}

			// If there is no static permutation then the base material is in the group path but isn't in the group yet.
			// In that case report the shader count from the base material shader count.
			const int32 NumShaders = bHasStaticPermutation ? UMaterialEditingLibrary::GetNumShaderTypes(MaterialInstance) : UMaterialEditingLibrary::GetNumShaderTypes(BaseMaterial);

			InContext.Message = FText::Format(
				LOCTEXT("MaterialEditorValidation_Warn", "This material instance creates a new permutation of Base Material '{0}'. It generates {1} shaders. Can you use or match an existing permutation?"),
				FText::FromString(BaseMaterial->GetName()),
				FText::AsNumber(NumShaders));

			TWeakObjectPtr<const UMaterialValidationGroup> WeakGroup(Group);
			FSoftObjectPath BaseMaterialPath(BaseMaterial);
			TWeakObjectPtr<UMaterialInstanceConstant> WeakInstance(MaterialInstance);
			InContext.HyperlinkText = LOCTEXT("MaterialEditorValidation_FindSimilar", "Find similar instances...");
			InContext.HyperlinkAction = FSimpleDelegate::CreateLambda([WeakGroup, BaseMaterialPath, WeakInstance]()
			{
				if (WeakGroup.IsValid() && WeakInstance.IsValid())
				{
					SMaterialInstanceSimilarityBrowser::CreateWindow(WeakGroup.Get(), BaseMaterialPath, WeakInstance.Get());
				}
			});

			return false;
		}
	}

	return true;
}

void FMaterialValidationHelpers::FetchRevisionHistory(TArray<UMaterialInterface*> const& InModifiedObjects)
{
	TArray<FString> Filenames;
	Filenames.Reserve(InModifiedObjects.Num());
	for (UMaterialInterface const* Object : InModifiedObjects)
	{
		if (Object != nullptr)
		{
			FString const PackageName = Object->GetOutermost()->GetName();
			FString Filename;
			if (FPackageName::TryConvertLongPackageNameToFilename(PackageName, Filename, FPackageName::GetAssetPackageExtension()))
			{
				Filenames.AddUnique(Filename);
			}
		}
	}

	if (Filenames.IsEmpty())
	{
		return;
	}

	TSharedRef<FUpdateStatus> UpdateOp = ISourceControlOperation::Create<FUpdateStatus>();
	UpdateOp->SetUpdateHistory(true);
	ISourceControlModule::Get().GetProvider().Execute(UpdateOp, Filenames, EConcurrency::Synchronous);
}

void FMaterialValidationHelpers::LoadDepotVersions(TArray<UMaterialInterface*> const& InModifiedObjects, TArray<UMaterialInterface*>& OutReplacementObjects, TArray<UPackage*>& OutDepotPackages)
{
	ISourceControlProvider& Provider = ISourceControlModule::Get().GetProvider();

	OutReplacementObjects.Reset(InModifiedObjects.Num());

	for (UMaterialInterface const* Object : InModifiedObjects)
	{
		UMaterialInterface* DepotVersion = nullptr;

		if (Object != nullptr)
		{
			FString const PackageName = Object->GetOutermost()->GetName();
			FString Filename;
			if (FPackageName::TryConvertLongPackageNameToFilename(PackageName, Filename, FPackageName::GetAssetPackageExtension()))
			{
				FSourceControlStatePtr HistoryState = Provider.GetState(Filename, EStateCacheUsage::Use);
				if (HistoryState.IsValid() && HistoryState->GetHistorySize() > 0)
				{
					TSharedPtr<ISourceControlRevision> Revision = HistoryState->GetHistoryItem(0);
					if (Revision.IsValid())
					{
						UPackage* DepotPackage = DiffUtils::LoadPackageForDiff(Revision);
						if (DepotPackage != nullptr)
						{
							OutDepotPackages.Add(DepotPackage);
							DepotVersion = Cast<UMaterialInterface>(DepotPackage->FindAssetInPackage());
						}
					}
				}
			}
		}

		OutReplacementObjects.Add(DepotVersion);
	}
}

#undef LOCTEXT_NAMESPACE
