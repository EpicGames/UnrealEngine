// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorValidatorBase.h"
#include "MaterialEditorValidation.h"

#include "MaterialValidators.generated.h"

class UMaterial;
class UMaterialInterface;
class UPackage;

/** Implementation of UEditorValidatorBase that applies validation for limiting the addition of material permutations from UMaterials. */
UCLASS()
class UEditorValidator_MaterialPermutation : public UEditorValidatorBase
{
	GENERATED_BODY()

	virtual bool CanValidateAsset_Implementation(const FAssetData& InAssetData, UObject* InAsset, FDataValidationContext& InContext) const override;
	virtual EDataValidationResult ValidateLoadedAsset_Implementation(const FAssetData& InAssetData, UObject* InAsset, FDataValidationContext& InContext) override;
	virtual void PostAssetValidation(TArray<TSharedRef<FTokenizedMessage>>& OutMessages) override;

private:
	/** Collected base materials for which to validate the full hierarchy after individual asset validation is complete. */
	UPROPERTY()
	TArray<TObjectPtr<UMaterial>> BaseMaterials;
	/** Collected individual assets whose changes should be taken into account when doing full hierarchy validation. */
	UPROPERTY()
	TArray<TObjectPtr<UMaterialInterface>> ModifiedObjects;
	/** Set when a material function is encountered. We skip the slow validation path because it is more likely to be very slow in this case. */
	bool bHasFunctionFiles = false;
};

/** Implementation of UMaterialEditorValidatorBase that applies validation for limiting the addition of material permutations. */
UCLASS()
class UMaterialEditorValidator_Permutation : public UMaterialEditorValidatorBase
{
	GENERATED_BODY()

	virtual bool Validate(UMaterialInterface* InMaterial, FMaterialEditorValidatorContext& InContext) const override;
};

/** Helper functions for external validation callers. */
struct FMaterialValidationHelpers
{
	/** Fetches source control revision history for the packages backing the given objects. */
	static void FetchRevisionHistory(TArray<UMaterialInterface*> const& InModifiedObjects);

	/** Loads the depot HEAD revision for each object. OutReplacementObjects is aligned to InModifiedObjects (nullptr where unavailable). */
	static void LoadDepotVersions(TArray<UMaterialInterface*> const& InModifiedObjects, TArray<UMaterialInterface*>& OutReplacementObjects, TArray<UPackage*>& OutDepotPackages);
};
