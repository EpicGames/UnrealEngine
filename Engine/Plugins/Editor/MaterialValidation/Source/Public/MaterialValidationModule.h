// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
#include "Modules/ModuleInterface.h"
#include "UObject/WeakObjectPtr.h"

DECLARE_LOG_CATEGORY_EXTERN(LogMaterialValidation, Log, All);

class SWindow;
struct FSoftObjectPath;
class UMaterialInterface;

class FMaterialValidationModule : public IModuleInterface
{
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

public:
	/**
	 * Open (or reuse) the similarity browser window for the given material hierarchy.
	 *
	 * @param MaterialInterface the material (instance) used for similarity. The parent material will be picked automatically
	 */
	MATERIALVALIDATION_API TSharedPtr<SWindow> OpenSimilarityBrowser(UMaterialInterface& MaterialInterface);

private:
	void OnEditorInitialized(double Duration);
	void OnEditorPreExit();

	TArray<TWeakObjectPtr<class UMaterialEditorValidatorBase>> MaterialEditorValidators;
	TArray<TWeakObjectPtr<class UEditorValidatorBase>> EditorValidators;
};
